{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.1",
	"FriendlyName": "CompositeCore",
	"Description": "Extensible core plugin for real-time compositing, with a default (holdout) composite pipeline through post-processing.",
	"Category": "Compositing",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "CompositeCore",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit"
		}
	]
}