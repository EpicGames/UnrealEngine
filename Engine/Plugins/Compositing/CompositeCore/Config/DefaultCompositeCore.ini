[CoreRedirects]

+ClassRedirects=(OldName="/Script/HoldoutComposite.HoldoutCompositeComponent", NewName="/Script/CompositeCore.HoldoutCompositeComponent")
+ClassRedirects=(OldName="/Script/HoldoutComposite.HoldoutCompositeSubsystem", NewName="/Script/CompositeCore.CompositeCoreSubsystem")
+ClassRedirects=(OldName="/Script/HoldoutComposite.HoldoutCompositeSettings", NewName="/Script/CompositeCore.CompositeCoreSettings")

+PropertyRedirects=(OldName="/Script/HoldoutComposite.HoldoutCompositeSettings.bCompositeFollowsSceneExposure", NewName="/Script/CompositeCore.CompositeCoreSettings.bApplyPreExposure")