// Copyright Epic Games, Inc. All Rights Reserved.

// Combined alpha coverage
float CoverageCombined(float As, float Ad)
{
	return As * Ad;
}

// Alpha coverage of source
float CoverageSrcOnly(float As, float Ad)
{
	return As * (1 - Ad);
}

// Alpha coverage of destination
float CoverageDstOnly(float As, float Ad)
{
	return Ad * (1 - As);
}

// Coverage weights: combined, source only, destination only.
float3 CoverageWeights(float As, float Ad)
{
	return float3(
		CoverageCombined(As, Ad),
		CoverageSrcOnly(As, Ad),
		CoverageDstOnly(As, Ad)
	);
}

// Obtain the base unpremultiplied color
float3 Unpremult(float4 Color)
{
	return select(Color.a > 0.0f, Color.rgb / Color.a, 0.0f);
}

// Screen alpha blend
float4 ScreenBlend( float4 Src, float4 Dst) 
{
	float3 W = CoverageWeights(Src.a, Dst.a);
	float3 SrcU = Unpremult(Src);
	float3 DstU = Unpremult(Dst);
	float3 Blend = SrcU + DstU - SrcU * DstU;
	
	return float4(
		Blend * W.x + SrcU * W.y + DstU * W.z,
		W.x + W.y + W.z
	);
}

// Overlay alpha blend
float4 OverlayBlend(float4 Src, float4 Dst) 
{
	float3 W = CoverageWeights(Src.a, Dst.a);
	float3 SrcU = Unpremult(Src);
	float3 DstU = Unpremult(Dst);
	float3 Blend = select(DstU <= 0.5f, 2 * SrcU * DstU, 1 - 2 * (1 - SrcU) * (1 - DstU));
	
	return float4(
		Blend * W.x + SrcU * W.y + DstU * W.z,
		W.x + W.y + W.z
	);
}

// Darken alpha blend
float4 DarkenBlend(float4 Src, float4 Dst) 
{
	float3 W = CoverageWeights(Src.a, Dst.a);
	float3 SrcU = Unpremult(Src);
	float3 DstU = Unpremult(Dst);
	float3 Blend = min(SrcU, DstU);
	
	return float4(
		Blend * W.x + SrcU * W.y + DstU * W.z,
		W.x + W.y + W.z
	);
}

// Lighten alpha blend
float4 LightenBlend(float4 Src, float4 Dst) 
{
	float3 W = CoverageWeights(Src.a, Dst.a);
	float3 SrcU = Unpremult(Src);
	float3 DstU = Unpremult(Dst);
	float3 Blend = max(SrcU, DstU);
	
	return float4(
		Blend * W.x + SrcU * W.y + DstU * W.z,
		W.x + W.y + W.z
	);
}

float ColorDodgeBlend(float Src, float Dst)
{
	if(Dst <= 0)
	{
		return 0;
	}
	else if(Dst > 0 && Dst < 1)
	{
		return select(Src < 1, min(1, Dst / (1 - Src)), 1.0f);
	}
	else
	{
		return 1;
	}
}

// Color dodge alpha blend
float4 ColorDodgeBlend(float4 Src, float4 Dst)
{
	float3 W = CoverageWeights(Src.a, Dst.a);
	float3 SrcU = Unpremult(Src);
	float3 DstU = Unpremult(Dst);
	float3 Blend = float3(
		ColorDodgeBlend(SrcU.r, DstU.r),
		ColorDodgeBlend(SrcU.g, DstU.g),
		ColorDodgeBlend(SrcU.b, DstU.b)
	);
	
	return float4(
		Blend * W.x + SrcU * W.y + DstU * W.z,
		W.x + W.y + W.z
	);
}

float ColorBurnBlend(float Src, float Dst)
{
	if(Dst <= 0)
	{
		return 0;
	}
	else if(Dst > 0 && Dst < 1)
	{
		return select(Src > 0, 1 - min(1, (1 - Dst) / Src), 0.0f);
	}
	else
	{
		return 1;
	}
}

// Color burn alpha blend
float4 ColorBurnBlend(float4 Src, float4 Dst)
{
	float3 W = CoverageWeights(Src.a, Dst.a);
	float3 SrcU = Unpremult(Src);
	float3 DstU = Unpremult(Dst);
	float3 Blend = float3(
		ColorBurnBlend(SrcU.r, DstU.r),
		ColorBurnBlend(SrcU.g, DstU.g),
		ColorBurnBlend(SrcU.b, DstU.b)
	);
	
	return float4(
		Blend * W.x + SrcU * W.y + DstU * W.z,
		W.x + W.y + W.z
	);
}

// Hard light alpha blend
float4 HardLightBlend(float4 Src, float4 Dst) 
{
	float3 W = CoverageWeights(Src.a, Dst.a);
	float3 SrcU = Unpremult(Src);
	float3 DstU = Unpremult(Dst);
	float3 Blend = select(SrcU <= 0.5, 2 * SrcU * DstU, 1.0 - 2.0 * (1.0 - SrcU) * (1.0 - DstU));
	
	return float4(
		Blend * W.x + SrcU * W.y + DstU * W.z,
		W.x + W.y + W.z
	);
}

float SoftLightBlend(float Src, float Dst)
{
	if(Src <= 0.5)
	{
		return Dst - (1 - 2 * Src) * Dst * (1 - Dst);
	}
	else if(Src > 0.5 && Dst <= 0.25)
	{
		return Dst + (2 * Src - 1) * Dst * ((16 * Dst - 12) * Dst + 3);
	}
	else
	{
		return Dst + (2 * Src - 1) * (sqrt(Dst) - Dst);
	}
}

// Soft light alpha blend
float4 SoftLightBlend(float4 Src, float4 Dst) 
{
	float3 W = CoverageWeights(Src.a, Dst.a);
	float3 SrcU = Unpremult(Src);
	float3 DstU = Unpremult(Dst);
	float3 Blend = float3(
		SoftLightBlend(SrcU.r, DstU.r),
		SoftLightBlend(SrcU.g, DstU.g),
		SoftLightBlend(SrcU.b, DstU.b)
	);
	
	return float4(
		Blend * W.x + SrcU * W.y + DstU * W.z,
		W.x + W.y + W.z
	);
}

// Difference alpha blend
float4 DifferenceBlend(float4 Src, float4 Dst) 
{
	float3 W = CoverageWeights(Src.a, Dst.a);
	float3 SrcU = Unpremult(Src);
	float3 DstU = Unpremult(Dst);
	float3 Blend = abs(DstU - SrcU);
	
	return float4(
		Blend * W.x + SrcU * W.y + DstU * W.z,
		W.x + W.y + W.z
	);
}

// Exclusion alpha blend
float4 ExclusionBlend(float4 Src, float4 Dst)
{
	float3 W = CoverageWeights(Src.a, Dst.a);
	float3 SrcU = Unpremult(Src);
	float3 DstU = Unpremult(Dst);
	float3 Blend = SrcU + DstU - 2 * SrcU * DstU;
	
	return float4(
		Blend * W.x + SrcU * W.y + DstU * W.z,
		W.x + W.y + W.z
	);
}
