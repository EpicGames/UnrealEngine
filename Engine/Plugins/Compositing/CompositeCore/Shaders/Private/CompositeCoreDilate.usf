// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/ScreenPass.ush"

#ifndef TILE_SIZE
    #error TILE_SIZE must be defined
#endif

#ifndef MORPHOLOGY_ERODE
    #define MORPHOLOGY_ERODE 0
#endif

// DILATION_CHANNELS_MODE:
//   0 = ALPHA_ONLY   - operate on alpha only (holdout system default, preserves today's behavior).
//   1 = GENERAL      - operate on any subset of R,G,B,A given by ChannelMask.
#define CHANNELS_MODE_ALPHA_ONLY 0
#define CHANNELS_MODE_GENERAL    1
#ifndef DILATION_CHANNELS_MODE
    #define DILATION_CHANNELS_MODE CHANNELS_MODE_ALPHA_ONLY
#endif

// DILATION_USE_THRESHOLD:
//   1 = thresholded morphology (today's behavior; when alpha is the target the neighbor's full
//       unpremultiplied RGB travels with the alpha flip, preserving the "spread color into
//       transparent" workflow).
//   0 = true min/max morphology (per-channel, graded output).
#ifndef DILATION_USE_THRESHOLD
    #define DILATION_USE_THRESHOLD 1
#endif

#define KERNEL_COUNT ((1 + 2 * DILATION_SIZE) * (1 + 2 * DILATION_SIZE) - 1)

// Number of texels around the group tile for scene color.
#define LDS_TILE_BORDER_SIZE DILATION_SIZE

// Width in texels of the depth tile cached into LDS.
#define LDS_TILE_WIDTH (TILE_SIZE + 2 * LDS_TILE_BORDER_SIZE)

// Total number of texels cached in the scene color tile.
#define LDS_ARRAY_SIZE (LDS_TILE_WIDTH * LDS_TILE_WIDTH)

#define THREADGROUP_TOTAL (TILE_SIZE * TILE_SIZE)

Texture2D Input_Texture;
SamplerState Input_Sampler;
SCREEN_PASS_TEXTURE_VIEWPORT(Input)
SCREEN_PASS_TEXTURE_VIEWPORT(Output)
RWTexture2D<float4> RWOutputTexture;
uint bInvertedAlpha;
uint bOpacifyOutput;
// Opacity at or below this threshold is treated as fully transparent for morphological operations.
// Read by the thresholded paths and by the Opacify step when bOpacifyOutput is set.
float AlphaThreshold;
// Bitmask selecting which channels the GENERAL path operates on (R=0x1, G=0x2, B=0x4, A=0x8).
uint ChannelMask;
// When true and Alpha is in the mask, the alpha-winning neighbor's RGB is copied into color
// channels that are NOT in the mask. Holdout defaults it to 0; the user pass defaults it to 1.
uint bCarryRGBWithAlpha;

// Normalize or restore alpha between inverted UE convention (A=0 opaque, A=1 transparent)
// and standard opacity (0 transparent, 1 opaque). Self-inverse: apply twice to round-trip.
// Defined inline - CompositeCommon.ush lives in the Composite plugin and is not accessible here.
float InvertAlpha(float InAlpha, uint bInvert)
{
    return select(bInvert > 0u, 1.0f - InAlpha, InAlpha);
}

#if DILATION_SIZE == 1
static const int2 kOffsets[KERNEL_COUNT] = {
    int2(-1,  0),
    int2( 1,  0),
    int2( 0,  1),
    int2( 0, -1),
    int2(-1,  1),
    int2( 1,  1),
    int2( 1, -1),
    int2(-1, -1),
};
#elif DILATION_SIZE == 2
static const int2 kOffsets[24] = {
    int2(-1,  0),
    int2( 1,  0),
    int2( 0,  1),
    int2( 0, -1),
    int2(-1,  1),
    int2( 1,  1),
    int2( 1, -1),
    int2(-1, -1),
    int2(-2,  0),
    int2( 2,  0),
    int2( 0, -2),
    int2( 0,  2),
    int2(-2, -1),
    int2(-1, -2),
    int2( 1, -2),
    int2( 2, -1),
    int2( 2,  1),
    int2( 1,  2),
    int2(-1,  2),
    int2(-2,  1),
    int2(-2, -2),
    int2( 2, -2),
    int2( 2,  2),
    int2(-2,  2),
};
#endif

groupshared float4 SharedColor[LDS_ARRAY_SIZE];

// Get the index within the LDS array.
uint GetTileArrayIndexFromPixelOffset(in uint2 GroupThreadId, in int2 PixelOffset)
{
    uint2 TilePos = GroupThreadId + uint2(PixelOffset + LDS_TILE_BORDER_SIZE);
    return TilePos.x + TilePos.y * (LDS_TILE_BORDER_SIZE * 2 + TILE_SIZE);
}

float4 SampleCachedColors(in uint2 GroupThreadId, int2 PixelOffset = int2(0,0))
{
    return SharedColor[GetTileArrayIndexFromPixelOffset(GroupThreadId, PixelOffset)];
}

float SafeRcp(float x)
{
    return x > 0.0 ? rcp(x) : 0.0;
}

// Unpremultiply and force full opacity. Operates in standard opacity space (a = 1 = opaque).
float4 Opacify(float4 InColor)
{
    InColor.xyz *= SafeRcp(saturate(InColor.a));
    return float4(InColor.xyz, 1.0f);
}

// ============================================================================
// Morphology helpers. The four combinations of DILATION_CHANNELS_MODE ×
// DILATION_USE_THRESHOLD dispatch to the helpers below. MORPHOLOGY_ERODE is a
// compile-time permutation so each helper resolves to a single code path.
// ============================================================================
#if DILATION_SIZE > 0

// Thresholded, alpha-coupled step: when the threshold is crossed, copy the *neighbor's full*
// unpremultiplied RGBA into TexelColor - this is what carries fringe colour outward with an
// alpha dilate in today's premultiplied workflow. Used by ALPHA_ONLY and by GENERAL when the
// A bit of ChannelMask is set.
float4 AlphaCoupledThresholdedStep(float4 TexelColor, uint2 GroupThreadId)
{
#if MORPHOLOGY_ERODE
    BRANCH
    if (TexelColor.a > 1.0f - AlphaThreshold)
    {
        UNROLL
        for (int i = 0; i < KERNEL_COUNT; i++)
        {
            float4 NeighborColor = SampleCachedColors(GroupThreadId, kOffsets[i]);
            if (InvertAlpha(NeighborColor.a, bInvertedAlpha) < AlphaThreshold)
            {
                TexelColor = float4(0.0f, 0.0f, 0.0f, 0.0f);
                break;
            }
        }
    }
#else
    BRANCH
    if (TexelColor.a < AlphaThreshold)
    {
        UNROLL
        for (int i = 0; i < KERNEL_COUNT; i++)
        {
            float4 OffsetColor = SampleCachedColors(GroupThreadId, kOffsets[i]);
            float OffsetOpacity = InvertAlpha(OffsetColor.a, bInvertedAlpha);
            if (OffsetOpacity > TexelColor.a)
            {
                TexelColor = float4(OffsetColor.rgb, OffsetOpacity);
                break;
            }
        }
    }
#endif
    return TexelColor;
}

// Thresholded step that only modifies alpha (no RGB copy). Used when bCarryRGBWithAlpha is off.
float AlphaOnlyThresholdedStep(float SelfAlpha, uint2 GroupThreadId)
{
#if MORPHOLOGY_ERODE
    BRANCH
    if (SelfAlpha > 1.0f - AlphaThreshold)
    {
        UNROLL
        for (int i = 0; i < KERNEL_COUNT; i++)
        {
            float na = InvertAlpha(SampleCachedColors(GroupThreadId, kOffsets[i]).a, bInvertedAlpha);
            if (na < AlphaThreshold)
            {
                return 0.0f;
            }
        }
    }
#else
    BRANCH
    if (SelfAlpha < AlphaThreshold)
    {
        UNROLL
        for (int i = 0; i < KERNEL_COUNT; i++)
        {
            float na = InvertAlpha(SampleCachedColors(GroupThreadId, kOffsets[i]).a, bInvertedAlpha);
            if (na > SelfAlpha)
            {
                return na;
            }
        }
    }
#endif
    return SelfAlpha;
}

// Per-channel thresholded step for a single R/G/B channel (Channel ∈ {0,1,2}).
// No RGB coupling: only the given channel's value is modified.
float PerChannelThresholdedStep(float SelfValue, uint2 GroupThreadId, uint Channel)
{
#if MORPHOLOGY_ERODE
    BRANCH
    if (SelfValue > 1.0f - AlphaThreshold)
    {
        UNROLL
        for (int i = 0; i < KERNEL_COUNT; i++)
        {
            float nv = SampleCachedColors(GroupThreadId, kOffsets[i])[Channel];
            if (nv < AlphaThreshold)
            {
                return 0.0f;
            }
        }
    }
#else
    BRANCH
    if (SelfValue < AlphaThreshold)
    {
        UNROLL
        for (int i = 0; i < KERNEL_COUNT; i++)
        {
            float nv = SampleCachedColors(GroupThreadId, kOffsets[i])[Channel];
            if (nv >= AlphaThreshold)
            {
                return nv;
            }
        }
    }
#endif
    return SelfValue;
}

// True min/max over the kernel for any selected channel (graded output). TexelColor.a must
// already be in standard opacity space; kernel neighbors are normalized inline. When Alpha is
// selected and bCarryRGBWithAlpha is set, the alpha-winning neighbor's RGB is copied into color
// channels that are NOT selected in the mask (so mattes expand visibly).
float4 MinMaxKernelGeneral(float4 TexelColor, uint2 GroupThreadId)
{
    // Track the alpha-winning neighbor's RGB for the post-loop carry fill.
    float3 AlphaWinnerRGB = TexelColor.rgb;
    float  BestAlpha      = TexelColor.a;

    UNROLL
    for (int i = 0; i < KERNEL_COUNT; i++)
    {
        float4 n = SampleCachedColors(GroupThreadId, kOffsets[i]);
        n.a = InvertAlpha(n.a, bInvertedAlpha);
    #if MORPHOLOGY_ERODE
        if ((ChannelMask & 0x1u) != 0u) TexelColor.r = min(TexelColor.r, n.r);
        if ((ChannelMask & 0x2u) != 0u) TexelColor.g = min(TexelColor.g, n.g);
        if ((ChannelMask & 0x4u) != 0u) TexelColor.b = min(TexelColor.b, n.b);
        if ((ChannelMask & 0x8u) != 0u)
        {
            if (n.a < BestAlpha) { BestAlpha = n.a; AlphaWinnerRGB = n.rgb; }
        }
    #else
        if ((ChannelMask & 0x1u) != 0u) TexelColor.r = max(TexelColor.r, n.r);
        if ((ChannelMask & 0x2u) != 0u) TexelColor.g = max(TexelColor.g, n.g);
        if ((ChannelMask & 0x4u) != 0u) TexelColor.b = max(TexelColor.b, n.b);
        if ((ChannelMask & 0x8u) != 0u)
        {
            if (n.a > BestAlpha) { BestAlpha = n.a; AlphaWinnerRGB = n.rgb; }
        }
    #endif
    }

    if ((ChannelMask & 0x8u) != 0u)
    {
        TexelColor.a = BestAlpha;
        if (bCarryRGBWithAlpha != 0u)
        {
            if ((ChannelMask & 0x1u) == 0u) TexelColor.r = AlphaWinnerRGB.r;
            if ((ChannelMask & 0x2u) == 0u) TexelColor.g = AlphaWinnerRGB.g;
            if ((ChannelMask & 0x4u) == 0u) TexelColor.b = AlphaWinnerRGB.b;
        }
    }

    return TexelColor;
}

#endif // DILATION_SIZE > 0

[numthreads(TILE_SIZE, TILE_SIZE, 1)]
void MainCS(
    uint2 GroupId : SV_GroupID,
    uint2 GroupThreadId : SV_GroupThreadID,
    uint GroupThreadIndex : SV_GroupIndex,
    uint2 DispatchThreadId : SV_DispatchThreadID)
{
    // Init LDS
    {
        // GroupTexelOffset must be signed: when GroupId.x or GroupId.y is 0 the halo extends one
        // pixel outside the viewport (position -1), which uint arithmetic would wrap to 0xFFFFFFFF.
        // The subsequent min-clamp would then map to ViewportSize-1 (the opposite edge) rather
        // than 0, loading pixels from the wrong corner and creating phantom opaque halos that look
        // like diagonal propagation artefacts at the viewport boundary.
        int2 GroupTexelOffsetSigned = int2(GroupId) * int2(TILE_SIZE, TILE_SIZE) - LDS_TILE_BORDER_SIZE;

        // To account for the dilation margin, each warp thread should load extra texels.
        // We proceed linearly through the tile (with borders) and each warp thread loads
        // multiple colors at THREADGROUP_TOTAL intervals with the necessary boundary conditions.
        const uint LoadCount = (LDS_ARRAY_SIZE + THREADGROUP_TOTAL - 1) / THREADGROUP_TOTAL; // integer ceiling

        uint LinearGroupThreadId = GroupThreadIndex;

        UNROLL
        for (uint i = 0; i < LoadCount; i++)
        {
            // Compute the signed viewport-relative texel position and clamp to [0, ViewportSize-1].
            int2 TexelLocationSigned = GroupTexelOffsetSigned + int2(
                int(LinearGroupThreadId % LDS_TILE_WIDTH),
                int(LinearGroupThreadId / LDS_TILE_WIDTH));
            TexelLocationSigned = clamp(TexelLocationSigned, int2(0, 0), int2(Input_ViewportSize) - 1);

            if ((LinearGroupThreadId < LDS_ARRAY_SIZE) ||
                (i != LoadCount - 1) ||
                (LDS_ARRAY_SIZE % THREADGROUP_TOTAL) == 0)
            {
                SharedColor[LinearGroupThreadId] = Input_Texture[uint2(TexelLocationSigned) + uint2(Input_ViewportMin)];
            }
            LinearGroupThreadId += THREADGROUP_TOTAL;
        }
    }

    GroupMemoryBarrierWithGroupSync();

    // Discard out-of-bound texels
    if (any(DispatchThreadId >= uint2(Output_ViewportSize)))
    {
        return;
    }

    float4 TexelColor = SampleCachedColors(GroupThreadId);
    // Normalize to standard opacity (0=transparent, 1=opaque) for all downstream logic.
    TexelColor.a = InvertAlpha(TexelColor.a, bInvertedAlpha);

#if DILATION_SIZE > 0
#if DILATION_CHANNELS_MODE == CHANNELS_MODE_ALPHA_ONLY
    // ALPHA_ONLY is only selected for the thresholded path (holdout + user-alpha-only+threshold);
    // the user pass routes all min/max dispatches through CHANNELS_MODE_GENERAL, so
    // ALPHA_ONLY + DILATION_USE_THRESHOLD=0 is excluded via ShouldCompilePermutation.
    TexelColor = AlphaCoupledThresholdedStep(TexelColor, GroupThreadId);
#else // CHANNELS_MODE_GENERAL
    #if DILATION_USE_THRESHOLD
        BRANCH
        if ((ChannelMask & 0x8u) != 0u)
        {
            if (bCarryRGBWithAlpha != 0u)
            {
                TexelColor = AlphaCoupledThresholdedStep(TexelColor, GroupThreadId);
            }
            else
            {
                TexelColor.a = AlphaOnlyThresholdedStep(TexelColor.a, GroupThreadId);
            }
        }
        UNROLL
        for (uint c = 0u; c < 3u; c++)
        {
            BRANCH
            if ((ChannelMask & (1u << c)) != 0u)
            {
                TexelColor[c] = PerChannelThresholdedStep(TexelColor[c], GroupThreadId, c);
            }
        }
    #else
        TexelColor = MinMaxKernelGeneral(TexelColor, GroupThreadId);
    #endif
#endif // DILATION_CHANNELS_MODE
#endif // DILATION_SIZE > 0

    // Conditionally & fully opacify non-translucent colors for the over blend.
    if (bOpacifyOutput && TexelColor.a > AlphaThreshold)
    {
        TexelColor = Opacify(TexelColor);
    }

    // Restore original alpha convention before writing.
    TexelColor.a = InvertAlpha(TexelColor.a, bInvertedAlpha);
    RWOutputTexture[Output_ViewportMin + DispatchThreadId] = TexelColor;
}
