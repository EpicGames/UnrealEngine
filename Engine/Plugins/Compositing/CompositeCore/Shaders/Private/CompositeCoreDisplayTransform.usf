// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/ScreenPass.ush"

FScreenTransform SvPositionToInputTextureUV;
Texture2D Input_Texture;
SamplerState Input_Sampler;

uint bIsForward;
float Gamma;
float InvGamma;

#define VERSION 0
#define R_K 1.0

// Simple Reinhard tone curve
float3 Reinhard(float3 ColorHDR, float K = R_K)
{
    return ColorHDR / (ColorHDR + K);
}

// Inverse simple Reinhard tone curve
float3 InvReinhard(float3 ColorSDR, float K = R_K)
{
    return K * ColorSDR / saturate(1.0 - ColorSDR);
}

// Gamma encode (faster approximation of sRGB)
float3 GammaEncode(float3 Color)
{
    return pow(Color, InvGamma);
}

// Gamma decode transfer function (faster approximation of sRGB)
float3 GammaDecode(float3 Color)
{
    return pow(Color, Gamma);
}

void MainPS(
    float4 SvPosition : SV_POSITION,
    out float4 OutColor : SV_Target0
    )
{
    float2 TextureUV = ApplyScreenTransform(SvPosition.xy, SvPositionToInputTextureUV);
    OutColor = Input_Texture.Sample(Input_Sampler, TextureUV);

#if VERSION == 0
    if(bIsForward)
    {
        OutColor.rgb = GammaEncode(Reinhard(OutColor.rgb));
    }
    else
    {
        OutColor.rgb = InvReinhard(GammaDecode(OutColor.rgb));
    }
#elif VERSION == 1
    OutColor.rgb = bIsForward ? Reinhard(OutColor.rgb) : InvReinhard(OutColor.rgb);
#elif VERSION == 2
    OutColor.rgb = bIsForward ? GammaEncode(OutColor.rgb) : GammaDecode(OutColor.rgb);
#endif
}
