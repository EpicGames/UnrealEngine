// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/GammaCorrectionCommon.ush"
#include "/Engine/Private/ScreenPass.ush"
#include "/Engine/Private/LensDistortion.ush"

#include "CompositeCoreBlending.ush"

#define ESourceEncoding_None  0
#define ESourceEncoding_Gamma 1
#define ESourceEncoding_sRGB  2

#define EDistortionUV_None        0
#define EDistortionUV_Distorted   1
#define EDistortionUV_Undistorted 2

#define ECompositeCoreMergeOp_None        0
#define ECompositeCoreMergeOp_Over        1
#define ECompositeCoreMergeOp_Under       2
#define ECompositeCoreMergeOp_Add         3
#define ECompositeCoreMergeOp_Subtract    4
#define ECompositeCoreMergeOp_Mul         5
#define ECompositeCoreMergeOp_Div         6
#define ECompositeCoreMergeOp_Min         7
#define ECompositeCoreMergeOp_Max         8
#define ECompositeCoreMergeOp_In          9
#define ECompositeCoreMergeOp_Mask        10
// Advanced blending modes
#define ECompositeCoreMergeOp_Screen      11
#define ECompositeCoreMergeOp_Overlay     12
#define ECompositeCoreMergeOp_Darken      13
#define ECompositeCoreMergeOp_Lighten     14
#define ECompositeCoreMergeOp_ColorDodge  15
#define ECompositeCoreMergeOp_ColorBurn   16
#define ECompositeCoreMergeOp_HardLight   17
#define ECompositeCoreMergeOp_SoftLight   18
#define ECompositeCoreMergeOp_Difference  19
#define ECompositeCoreMergeOp_Exclusion   20

SCREEN_PASS_TEXTURE_VIEWPORT(Input0)
SCREEN_PASS_TEXTURE_VIEWPORT(Input1)
SCREEN_PASS_TEXTURE_VIEWPORT(Output)

#define INPUT_PASS_TEXTURE_MEMBER(StructName, MemberType, MemberName) MemberType StructName##_##MemberName;

#define INPUT_PASS_TEXTURE(StructName) \
	INPUT_PASS_TEXTURE_MEMBER(StructName, Texture2D, Texture) \
	INPUT_PASS_TEXTURE_MEMBER(StructName, SamplerState, Sampler) \
    INPUT_PASS_TEXTURE_MEMBER(StructName, uint, bInvertedAlpha) \
    INPUT_PASS_TEXTURE_MEMBER(StructName, uint, SourceEncoding) \
    INPUT_PASS_TEXTURE_MEMBER(StructName, uint, DistortionUV) \
    INPUT_PASS_TEXTURE_MEMBER(StructName, float, Exposure)
    

INPUT_PASS_TEXTURE(Tex0);
INPUT_PASS_TEXTURE(Tex1);

Texture2D<float2> DistortingDisplacementTexture;
SamplerState DistortingDisplacementSampler;
Texture2D<float2> UndistortingDisplacementTexture;
SamplerState UndistortingDisplacementSampler;

float2 DisplayGamma; // DisplayGamma, 1/DisplayGamma
uint MergeOp;
uint OutputEncoding;
uint bFlags; // 0: bInvertedAlphaOutput
uint bBlendInDisplaySpace;
uint bLuminanceOnly; // Divide-only: when set, the matte's RGB uses a single luminance ratio uniformly across all channels.

float3 Decode(float3 InColor, uint InEncoding)
{
    // TODO: HDR cases will need to be handled explicitely.
    BRANCH
    if(InEncoding == ESourceEncoding_Gamma)
    {
        return pow(InColor, DisplayGamma.x);
    }
    else if(InEncoding == ESourceEncoding_sRGB)
    {
        return sRGBToLinear(InColor);
    }
    else
    {
        return InColor;
    }
}

float3 Encode(float3 InColor, uint InEncoding)
{
    BRANCH
    if(InEncoding == ESourceEncoding_Gamma)
    {
        return pow(InColor, DisplayGamma.y);
    }
    else if(InEncoding == ESourceEncoding_sRGB)
    {
        return LinearToSrgb(InColor);
    }
    else
    {
        return InColor;
    }
}

float2 CalcInputUVs(float2 ViewportUV, float2 ViewportSize, float2 ViewportMin, float2 ExtentInverse, uint DistortionUVType)
{
#if USE_DISTORTION
    BRANCH
    if(DistortionUVType == EDistortionUV_Distorted)
    {
        ViewportUV = ApplyLensDistortionOnViewportUV(DistortingDisplacementTexture, DistortingDisplacementSampler, ViewportUV);
    }
    else if(DistortionUVType == EDistortionUV_Undistorted)
    {
        ViewportUV = ApplyLensDistortionOnViewportUV(UndistortingDisplacementTexture, UndistortingDisplacementSampler, ViewportUV);
    }
#endif

    return (ViewportUV * ViewportSize + ViewportMin) * ExtentInverse;
}

float GetOpacity(float InAlpha, uint bInvertedAlpha)
{
	return select(bInvertedAlpha > 0, 1.0 - InAlpha, InAlpha);
}

float SafeDiv(float A, float B)
{
    return B == 0.0 ? 0.0 : A / B;
}

float4 SafeDiv(float4 A, float4 B)
{
    return float4(
        SafeDiv(A[0], B[0]),
        SafeDiv(A[1], B[1]),
        SafeDiv(A[2], B[2]),
        SafeDiv(A[3], B[3])
    );
}

// Biased ratio: pulls (A + eps) / (B + eps) toward 1.0 in low-signal regions to suppress speckle.
float4 BiasedDiv(float4 A, float4 B, float Eps)
{
    return max(0.0, (A + Eps) / (B + Eps));
}

// Encode RGB to sRGB for display-space blending; un-premultiplies first so the operand at blend is encode(straight)*alpha rather than encode(straight*alpha) - sRGB encoding is non-linear so the two diverge for fractional alpha. Inputs are expected linear-premult (engine convention; masking-pass output for Ultimatte content is converted upstream via the Ultimatte Blend option on the masking pass).
float3 EncodeForDisplayBlend(float4 Color)
{
    return LinearToSrgb(Unpremult(Color)) * Color.a;
}

// Inverse of EncodeForDisplayBlend: decode the post-blend display-space result back to linear-premult.
float3 DecodeFromDisplayBlend(float4 Color)
{
    return sRGBToLinear(Unpremult(Color)) * Color.a;
}

void MainPS(
    float4 SvPosition : SV_POSITION,
    out float4 OutColor : SV_Target0
    )
{
    float2 ViewportUV = (SvPosition.xy - Output_ViewportMin) * Output_ViewportSizeInverse;

    // Calculate texture UVs, with/without distortion
    float2 Tex0_UV = CalcInputUVs(ViewportUV, Input0_ViewportSize, Input0_ViewportMin, Input0_ExtentInverse, Tex0_DistortionUV);
    float2 Tex1_UV = CalcInputUVs(ViewportUV, Input1_ViewportSize, Input1_ViewportMin, Input1_ExtentInverse, Tex1_DistortionUV);

    // Sample textures
    float4 Color0 = Tex0_Texture.SampleLevel(Tex0_Sampler, Tex0_UV, 0);
    float4 Color1 = Tex1_Texture.SampleLevel(Tex1_Sampler, Tex1_UV, 0);

    // Linearize source colors
    Color0.rgb = Decode(Color0.rgb, Tex0_SourceEncoding);
    Color1.rgb = Decode(Color1.rgb, Tex1_SourceEncoding);

    // Normalize alpha as opacity
	Color0.a = GetOpacity(Color0.a, Tex0_bInvertedAlpha);
	Color1.a = GetOpacity(Color1.a, Tex1_bInvertedAlpha);

    // Adjust exposure if necessary
    Color0.rgb *= Tex0_Exposure;
    Color1.rgb *= Tex1_Exposure;
    
    // Optionally encode both inputs to sRGB for display-space blending (e.g. Ultimatte Key & Fill).
    BRANCH
    if(bBlendInDisplaySpace)
    {
        Color0.rgb = EncodeForDisplayBlend(Color0);
        Color1.rgb = EncodeForDisplayBlend(Color1);
    }

    BRANCH
    if(MergeOp == ECompositeCoreMergeOp_Over)
    {
        OutColor = Color0 + Color1 * (1.0 - Color0.a);
    }
	else if (MergeOp == ECompositeCoreMergeOp_Under)
	{
		OutColor = Color0 * (1.0 - Color1.a) + Color1;
	}
    else if(MergeOp == ECompositeCoreMergeOp_Add)
    {
        OutColor = Color0 + Color1;
    }
    else if(MergeOp == ECompositeCoreMergeOp_Subtract)
    {
        OutColor = Color0 - Color1;
    }
    else if(MergeOp == ECompositeCoreMergeOp_Mul)
    {
        OutColor = Color0 * Color1;
    }
    else if(MergeOp == ECompositeCoreMergeOp_Div)
    {
        const float DivisionEpsilon = 1e-3;

        BRANCH
        if (bLuminanceOnly)
        {
            // Single luminance ratio applied uniformly across RGB; per-channel ratio for alpha to preserve holdout semantics.
            float LumA = Luminance(Color0.rgb);
            float LumB = Luminance(Color1.rgb);
            OutColor.rgb = max(0.0, (LumA + DivisionEpsilon) / (LumB + DivisionEpsilon));
            OutColor.a = max(0.0, (Color0.a + DivisionEpsilon) / (Color1.a + DivisionEpsilon));
        }
        else
        {
            OutColor = BiasedDiv(Color0, Color1, DivisionEpsilon);
        }
    }
    else if(MergeOp == ECompositeCoreMergeOp_Min)
    {
        OutColor = min(Color0, Color1);
    }
    else if(MergeOp == ECompositeCoreMergeOp_Max)
    {
        OutColor = max(Color0, Color1);
    }
	else if (MergeOp == ECompositeCoreMergeOp_In)
	{
		OutColor = Color0 * Color1.a;
	}
	else if (MergeOp == ECompositeCoreMergeOp_Mask)
	{
		OutColor = Color1 * Color0.a;
	}
    //~ Start advanced blending modes
    else if(MergeOp == ECompositeCoreMergeOp_Screen)
    {
        OutColor = ScreenBlend(Color0, Color1);
    }
    else if(MergeOp == ECompositeCoreMergeOp_Overlay)
    {
        OutColor = OverlayBlend(Color0, Color1);
    }
    else if(MergeOp == ECompositeCoreMergeOp_Darken)
    {
        OutColor = DarkenBlend(Color0, Color1);
    }
    else if(MergeOp == ECompositeCoreMergeOp_Lighten)
    {
        OutColor = LightenBlend(Color0, Color1);
    }
    else if(MergeOp == ECompositeCoreMergeOp_ColorDodge)
    {
        OutColor = ColorDodgeBlend(Color0, Color1);
    }
    else if(MergeOp == ECompositeCoreMergeOp_ColorBurn)
    {
        OutColor = ColorBurnBlend(Color0, Color1);
    }
    else if(MergeOp == ECompositeCoreMergeOp_HardLight)
    {
        OutColor = HardLightBlend(Color0, Color1);
    }
    else if(MergeOp == ECompositeCoreMergeOp_SoftLight)
    {
        OutColor = SoftLightBlend(Color0, Color1);
    }
    else if(MergeOp == ECompositeCoreMergeOp_Difference)
    {
        OutColor = DifferenceBlend(Color0, Color1);
    }
    else if(MergeOp == ECompositeCoreMergeOp_Exclusion)
    {
        OutColor = ExclusionBlend(Color0, Color1);
    }
    //~ End advanced blending modes
    else
    {
        OutColor = Color0;
    }

    // Decode back from sRGB to linear after display-space blending.
    BRANCH
    if(bBlendInDisplaySpace)
    {
        OutColor.rgb = DecodeFromDisplayBlend(OutColor);
    }

    // Re-encode color
    OutColor.rgb = Encode(OutColor.rgb, OutputEncoding);

	const bool bInvertedAlphaOutput = (bFlags & 1);
	OutColor.a = saturate(select(bInvertedAlphaOutput, 1.0 - OutColor.a, OutColor.a));
}
