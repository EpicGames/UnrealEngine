// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CompositeCore : ModuleRules
{
	public CompositeCore(ReadOnlyTargetRules Target) : base(Target)
	{
		// The CompositeCore plugin is distributed with engine hot fixes and thus isn't tied to binary
		// compatibility between hotfixes by only using Public/ interface of the renderer, but also Internal/ ones.
		bTreatAsEngineModule = true;
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"DeveloperSettings",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Projects",
				"RenderCore",
				"Renderer",
				"RHI",
			}
		);

		if (Target.bBuildEditor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"UnrealEd",
					"SettingsEditor",
					"Slate",
					"SlateCore",
					"SourceControl",
				}
			);
		}

		//bUseUnity = false;
		//OptimizeCode = CodeOptimization.Never;
		//PCHUsage = PCHUsageMode.NoPCHs;
	}
}
