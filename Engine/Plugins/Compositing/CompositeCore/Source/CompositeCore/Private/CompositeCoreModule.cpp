// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositeCoreModule.h"

#include "Interfaces/IPluginManager.h"
#include "HAL/IConsoleManager.h"
#include "HDRHelper.h"
#include "Misc/ConfigUtilities.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "ShaderCore.h"

DEFINE_LOG_CATEGORY(LogCompositeCore);

#define LOCTEXT_NAMESPACE "CompositeCore"

void FCompositeCoreModule::StartupModule()
{
	AddShaderSourceDirectoryMapping(TEXT("/Plugin/CompositeCore"), FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("CompositeCore"))->GetBaseDir(), TEXT("Shaders")));

	// Since we are so early in the loading phase, we first need to load the cvars since they're not loaded at this point.
	UE::ConfigUtilities::ApplyCVarSettingsFromIni(TEXT("/Script/CompositeCore.CompositeCorePluginSettings"), *GEngineIni, ECVF_SetByProjectSetting);

	if (IsHDREnabled())
	{
		UE_LOGF(LogCompositeCore, Warning, "Composite pipeline disabled: HDR mode is not currently supported.");
	}
}

void FCompositeCoreModule::ShutdownModule()
{
}

IMPLEMENT_MODULE( FCompositeCoreModule, CompositeCore )

#undef LOCTEXT_NAMESPACE
