// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositeCoreSceneViewExtension.h"

#include "CompositeCoreModule.h"
#include "Passes/CompositeCorePassDilate.h"
#include "Passes/CompositeCorePassFXAAProxy.h"
#include "Passes/CompositeCorePassMergeProxy.h"

#include "CommonRenderResources.h"
#include "Components/PrimitiveComponent.h"
#include "Components/SceneCaptureComponent2D.h"
#include "Containers/Set.h"
#include "EngineUtils.h"
#include "Engine/Texture.h"
#include "HDRHelper.h"
#include "PostProcess/PostProcessAA.h"
#include "PostProcess/PostProcessMaterialInputs.h"
#include "Rendering/CustomRenderPass.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "SceneRenderTargetParameters.h"
#include "SceneView.h"
#include "ScreenPass.h"
#include "SystemTextures.h"
#include "TextureResource.h"

static TAutoConsoleVariable<int32> CVarCompositeCoreApplyFXAA(
	TEXT("CompositeCore.ApplyFXAA"),
	0,
	TEXT("When enabled, the custom render pass automatically applies FXAA."),
	ECVF_RenderThreadSafe);

class FCompositeCoreCustomRenderPass : public FCustomRenderPassBase
{
public:
	IMPLEMENT_CUSTOM_RENDER_PASS(FCompositeCoreCustomRenderPass);

	FCompositeCoreCustomRenderPass(const FIntPoint& InRenderTargetSize, FCompositeCoreSceneViewExtension* InParentExtension, const FSceneView& InView, const UE::CompositeCore::FBuiltInRenderPassOptions& InOptions)
		: FCustomRenderPassBase(TEXT("CompositeCoreCustomRenderPass"), ERenderMode::DepthAndBasePass, ERenderOutput::SceneColorAndAlpha, InRenderTargetSize)
		, ParentExtension(InParentExtension)
		, ViewId(InView.GetViewKey())
		, ViewFeatureLevel(InView.GetFeatureLevel())
		, Inputs({ InOptions.DilationSize, InOptions.bOpacifyOutput })
	{
		bSceneColorWithTranslucent = true;
	}

	virtual void OnPreRender(FRDGBuilder& GraphBuilder) override
	{
		const FRDGTextureDesc TextureDesc = FRDGTextureDesc::Create2D(RenderTargetSize, PF_FloatRGBA, FClearValueBinding::Black, TexCreate_RenderTargetable | TexCreate_ShaderResource);
		RenderTargetTexture = GraphBuilder.CreateTexture(TextureDesc, TEXT("CompositeCoreCustomTexture"));
		AddClearRenderTargetPass(GraphBuilder, RenderTargetTexture, FLinearColor::Black, FIntRect(FInt32Point(), RenderTargetSize));
	}

	virtual void OnPostRender(FRDGBuilder& GraphBuilder) override
	{
		const FRDGTextureDesc TextureDesc = FRDGTextureDesc::Create2D(RenderTargetTexture->Desc.Extent, PF_FloatRGBA, FClearValueBinding::Black, TexCreate_UAV | TexCreate_ShaderResource | TexCreate_RenderTargetable);
		FRDGTextureRef Output = GraphBuilder.CreateTexture(TextureDesc, TEXT("CompositeCoreProcessedTexture"));

		UE::CompositeCore::Private::AddDilatePass(GraphBuilder, RenderTargetTexture, Output, ViewFeatureLevel, Inputs);

		ParentExtension->CollectCustomRenderTarget(ViewId, GraphBuilder.ConvertToExternalTexture(Output));
	}

private:

	FCompositeCoreSceneViewExtension* ParentExtension;
	const uint32 ViewId;
	const ERHIFeatureLevel::Type ViewFeatureLevel;
	const UE::CompositeCore::Private::FDilateInputs Inputs;
};

FCompositeCoreSceneViewExtension::FCompositeCoreSceneViewExtension(const FAutoRegister& AutoReg, UWorld* InWorld)
	: FWorldSceneViewExtension(AutoReg, InWorld)
{
}

FCompositeCoreSceneViewExtension::~FCompositeCoreSceneViewExtension() = default;

void FCompositeCoreSceneViewExtension::RegisterPrimitives_GameThread(const TArray<UPrimitiveComponent*>& InPrimitiveComponents, ECompositeCoreHoldoutManagement HoldoutManagement)
{
	check(IsInGameThread());

	for (UPrimitiveComponent* InPrimitiveComponent : InPrimitiveComponents)
	{
		if (!IsValid(InPrimitiveComponent))
		{
			continue;
		}

		if (!CompositePrimitives.Contains(InPrimitiveComponent))
		{
			CompositePrimitives.Add(InPrimitiveComponent);
		}

		if (HoldoutManagement == ECompositeCoreHoldoutManagement::Automatic && !InPrimitiveComponent->bHoldout)
		{
			InPrimitiveComponent->Modify();
			InPrimitiveComponent->SetHoldout(true);
		}
	}
}

void FCompositeCoreSceneViewExtension::UnregisterPrimitives_GameThread(const TArray<UPrimitiveComponent*>& InPrimitiveComponents, ECompositeCoreHoldoutManagement HoldoutManagement)
{
	check(IsInGameThread());

	for (UPrimitiveComponent* InPrimitiveComponent : InPrimitiveComponents)
	{
		if (!IsValid(InPrimitiveComponent))
		{
			continue;
		}

		if (CompositePrimitives.Contains(InPrimitiveComponent))
		{
			CompositePrimitives.Remove(InPrimitiveComponent);
		}

		if (HoldoutManagement == ECompositeCoreHoldoutManagement::Automatic && InPrimitiveComponent->bHoldout)
		{
			InPrimitiveComponent->Modify();
			InPrimitiveComponent->SetHoldout(false);
		}
	}
}

void FCompositeCoreSceneViewExtension::SetRenderWork_GameThread(UE::CompositeCore::FRenderWork&& InWork)
{
	bHasCustomRenderWork = !InWork.IsEmpty();
	MainRenderMode = MoveTemp(InWork.MainRenderMode);
	// View modes are read on both threads (see IsAllowedForViewFamily); the active-viewport filter is only read on game thread (SetupView).
	AllowedViewModes = InWork.AllowedViewModes;
	ActiveViewportFilter = InWork.ActiveViewportFilter;
	SceneCapturesUpdateQueue = MoveTemp(InWork.SceneCapturesUpdateQueue);

	// External textures are intentionally NOT snapshotted here. The underlying FRHITexture is
	// resolved per-frame in FScopedExternalTextureMap so it picks up media-texture / render-target
	// updates that happen later in the same frame (e.g., FMediaTextureResource::Render runs from
	// IMediaModule::TickPostEngine, after ACompositeActor::Tick has already enqueued this command).
	ENQUEUE_RENDER_COMMAND(CopyCompositeCoreRenderWork)(
		[FrameWork = MoveTemp(InWork), WeakThis = AsWeak()](FRHICommandListImmediate& RHICmdList) mutable
		{
			TSharedPtr<FCompositeCoreSceneViewExtension> SVE = StaticCastSharedPtr<FCompositeCoreSceneViewExtension>(WeakThis.Pin());
			if (SVE.IsValid())
			{
				SVE->RenderWork_RenderThread = MoveTemp(FrameWork);
			}
		});
}

void FCompositeCoreSceneViewExtension::ResetRenderWork_GameThread()
{
	bHasCustomRenderWork = false;
	MainRenderMode.Reset();
	AllowedViewModes.Reset();
	ActiveViewportFilter.Reset();
	SceneCapturesUpdateQueue.Reset();

	ENQUEUE_RENDER_COMMAND(CopyCompositeCoreRenderWork)(
		[WeakThis = AsWeak()](FRHICommandListImmediate& RHICmdList)
		{
			TSharedPtr<FCompositeCoreSceneViewExtension> SVE = StaticCastSharedPtr<FCompositeCoreSceneViewExtension>(WeakThis.Pin());
			if (SVE.IsValid())
			{
				SVE->RenderWork_RenderThread.Reset();
			}
		});
}

void FCompositeCoreSceneViewExtension::SetBuiltInRenderPassOptions_GameThread(const UE::CompositeCore::FBuiltInRenderPassOptions& InOptions)
{
	BuiltInRenderPassOptions = InOptions;
}

void FCompositeCoreSceneViewExtension::ResetBuiltInRenderPassOptions_GameThread()
{
	BuiltInRenderPassOptions.Reset();
}

/* Called by the custom render pass to store its view render target for this frame. */

void FCompositeCoreSceneViewExtension::CollectCustomRenderTarget(uint32 InViewId, const TRefCountPtr<IPooledRenderTarget>& InRenderTarget)
{
	CustomRenderTargetPerView_RenderThread.Add(InViewId, InRenderTarget);
}

bool FCompositeCoreSceneViewExtension::IsActiveThisFrame_Internal(const FSceneViewExtensionContext& Context) const
{
	bool bIsActive = FWorldSceneViewExtension::IsActiveThisFrame_Internal(Context);
	bIsActive &= bHasCustomRenderWork || !CompositePrimitives.IsEmpty();
	bIsActive &= !IsHDREnabled();

	return bIsActive;
}


//~ Begin ISceneViewExtension Interface

int32 FCompositeCoreSceneViewExtension::GetPriority() const
{
	return GetDefault<UCompositeCorePluginSettings>()->SceneViewExtensionPriority;
}

const UE::CompositeCore::FRenderWork& FCompositeCoreSceneViewExtension::GetRenderWork() const
{
	if (RenderWork_RenderThread.IsSet())
	{
		return RenderWork_RenderThread.GetValue();
	}
	else
	{
		return UE::CompositeCore::FRenderWork::GetDefault();
	}
}

void FCompositeCoreSceneViewExtension::SetupViewFamily(FSceneViewFamily& InViewFamily)
{
	// Cleanup invalid primitives.
	for (auto Iter = CompositePrimitives.CreateIterator(); Iter; ++Iter)
	{
		if (!Iter->IsValid())
		{
			Iter.RemoveCurrent();
		}
	}
}

void FCompositeCoreSceneViewExtension::SetupView(FSceneViewFamily& InViewFamily, FSceneView& InView)
{
	const bool bIsFirstView = (&InView == InViewFamily.Views[0]);
	if (!bIsFirstView)
	{
		return;
	}

	auto IsActiveViewportFilterMatchFn = [this, &InViewFamily]() -> bool
		{
			if (!ActiveViewportFilter.IsSet())
			{
				return true;
			}

#if WITH_EDITOR
			const UE::CompositeCore::FActiveViewportFilter& Filter = ActiveViewportFilter.GetValue();

			// Active level viewport render target (pointer comparison only).
			if (Filter.TargetRenderTarget != nullptr && InViewFamily.RenderTarget == Filter.TargetRenderTarget)
			{
				return true;
			}

			// Target actor unique ID (stays correct under WITH_STATE_STREAM, where FSceneViewOwner hides its pointer).
			if (const AActor* TargetActor = Filter.TargetViewActor.Get())
			{
				const uint32 TargetActorId = TargetActor->GetUniqueID();
				for (const FSceneView* View : InViewFamily.Views)
				{
					if (View != nullptr && View->ViewActor.ActorUniqueId == TargetActorId)
					{
						return true;
					}
				}
			}

			return false;
#else
			// No "active viewport" concept outside the editor; allow all eligible families.
			return true;
#endif
		};

	// The family view mode is only accessible after SetupViewFamily is called currently.
	const bool bViewModeAllowed = AllowedViewModes.IsEmpty() || AllowedViewModes.Contains(InViewFamily.ViewMode);
	const bool bViewportAllowed = IsActiveViewportFilterMatchFn();

	if (!bViewModeAllowed || !bViewportAllowed)
	{
		// Clear AllowPrimitiveAlphaHoldout so registered holdouts don't punch black silhouettes
		// in viewports where the merge pass will not run. IsAllowedForViewFamily reads this flag,
		// so the same clear also skips BeginRenderViewFamily and the post-processing subscription.
		InViewFamily.EngineShowFlags.SetAllowPrimitiveAlphaHoldout(false);
	}
	else if (!InView.bIsSceneCapture)
	{
		// Manually update scene captures that have been registered but aren't already updating every frame
		for (TWeakObjectPtr<USceneCaptureComponent2D>& SceneCapture : SceneCapturesUpdateQueue)
		{
			if (SceneCapture.IsValid() && !SceneCapture->bCaptureEveryFrame)
			{
				SceneCapture->CaptureSceneDeferred();
			}
		}

		// Empty update queue
		SceneCapturesUpdateQueue.Reset();
	}
}

void FCompositeCoreSceneViewExtension::BeginRenderViewFamily(FSceneViewFamily& InViewFamily)
{
	if (!IsAllowedForViewFamily(InViewFamily))
	{
		return;
	}

	if (MainRenderMode.IsSet())
	{
		InViewFamily.SceneCaptureSource = MainRenderMode.GetValue();
	}

	const TWeakObjectPtr<UWorld> WorldPtr = GetWorld();
	check(WorldPtr.IsValid());

	for (int32 ViewIndex = 0; ViewIndex < InViewFamily.Views.Num(); ++ViewIndex)
	{
		const FSceneView& InView = *InViewFamily.Views[ViewIndex];

		TSet<FPrimitiveComponentId> CompositeCorePrimitiveIds;
		for (const TWeakObjectPtr<UPrimitiveComponent>& PrimitivePtr : CompositePrimitives)
		{
			TStrongObjectPtr<UPrimitiveComponent> Primitive = PrimitivePtr.Pin();
			// Collect only those primitives that use the bHoldout flag
			// The user can directly change this flag outside of this VE.
			if (Primitive.IsValid() && Primitive->bHoldout)
			{
				const FPrimitiveComponentId PrimId = Primitive->GetPrimitiveSceneId();

				if (InView.ShowOnlyPrimitives.IsSet())
				{
					if (InView.ShowOnlyPrimitives.GetValue().Contains(PrimId))
					{
						CompositeCorePrimitiveIds.Add(Primitive->GetPrimitiveSceneId());
					}
				}
				else if (!InView.HiddenPrimitives.Contains(PrimId))
				{
					CompositeCorePrimitiveIds.Add(Primitive->GetPrimitiveSceneId());
				}
			}
		}

		if (CompositeCorePrimitiveIds.IsEmpty())
		{
			continue;
		}

		UE::CompositeCore::FBuiltInRenderPassOptions RenderPassOptions = BuiltInRenderPassOptions.IsSet() ? *BuiltInRenderPassOptions : UE::CompositeCore::FBuiltInRenderPassOptions{};

		// Create a new custom render pass to render the composite primitive(s)
		FCompositeCoreCustomRenderPass* CustomRenderPass = new FCompositeCoreCustomRenderPass(
			InView.UnscaledViewRect.Size(),
			this,
			InView,
			RenderPassOptions
		);

		FSceneInterface::FCustomRenderPassRendererInput PassInput{};
		PassInput.EngineShowFlags = InViewFamily.EngineShowFlags;
		PassInput.EngineShowFlags.DisableFeaturesForUnlit();
		PassInput.EngineShowFlags.SetTranslucency(true);
		PassInput.EngineShowFlags.SetUnlitViewmode(RenderPassOptions.bEnableUnlitViewmode);
		PassInput.EngineShowFlags.SetAllowPrimitiveAlphaHoldout(false);
		if (RenderPassOptions.ViewUserFlagsOverride.IsSet())
		{
			PassInput.bOverridesPostVolumeUserFlags = true;
			PassInput.PostVolumeUserFlags = RenderPassOptions.ViewUserFlagsOverride.GetValue();
		}
		// Note: Incoming view location is invalid for scene captures
		PassInput.ViewLocation = InView.ViewMatrices.GetViewOrigin();
		PassInput.ViewRotationMatrix = InView.ViewMatrices.GetWorldToView().RemoveTranslation();
		PassInput.ViewRotationMatrix.RemoveScaling();

		// Note: Projection matrix here is without jitter, GetViewToClipNoAA() is invalid (not yet available).
		PassInput.ProjectionMatrix = InView.ViewMatrices.GetViewToClip();
		PassInput.ViewActor = InView.ViewActor;
		PassInput.ShowOnlyPrimitives = CompositeCorePrimitiveIds;
		PassInput.CustomRenderPass = CustomRenderPass;
		PassInput.bIsSceneCapture = true;

		WorldPtr.Get()->Scene->AddCustomRenderPass(&InViewFamily, PassInput);
	}
}

void FCompositeCoreSceneViewExtension::PreRenderViewFamily_RenderThread(FRDGBuilder& GraphBuilder, FSceneViewFamily& InViewFamily)
{
	// To prevent cache reallocations each frame, we keep the existing views unless they start exceeding a reasonable number.
	constexpr int32 MaxNumCachedViews = 16;

	if (CachedOutputsPerView_RenderThread.Num() <= MaxNumCachedViews)
	{
		for (auto& Pair : CachedOutputsPerView_RenderThread)
		{
			// Reset each pre-cached view's output
			Pair.Value.Reset();
		}
	}
	else
	{
		CachedOutputsPerView_RenderThread.Reset();
	}
}

void FCompositeCoreSceneViewExtension::PreRenderView_RenderThread(FRDGBuilder& GraphBuilder, FSceneView& InView)
{
	static uint64 LastFrameCounter = 0;

	// We only apply pre-processing once per frame, on the very first render.
	if (LastFrameCounter != GFrameCounterRenderThread)
	{
		ApplyPreprocessing(GraphBuilder, InView);
		
		LastFrameCounter = GFrameCounterRenderThread;
	}
}

void FCompositeCoreSceneViewExtension::PrePostProcessPass_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& InView, const FPostProcessingInputs& Inputs)
{
	using namespace UE::CompositeCore;

	TRefCountPtr<IPooledRenderTarget>* CompositeRenderPassPtr = CustomRenderTargetPerView_RenderThread.Find(InView.GetViewKey());

	if ((CompositeRenderPassPtr != nullptr) && CVarCompositeCoreApplyFXAA.GetValueOnRenderThread())
	{
		static FFXAAPassProxy FXAAPassProxy = FFXAAPassProxy(GetDefaultInputDeclArray());

		// Set the composite render target as the pass input
		FPassInputArray PassInputs;
		FPassInput& PassInput = PassInputs.GetInputs().AddDefaulted_GetRef();
		PassInput.Texture = FScreenPassTexture{ GraphBuilder.RegisterExternalTexture(*CompositeRenderPassPtr) };

		// Apply FXAA, with additional fwd/inv display transform passes
		const FPassTexture Output = FXAAPassProxy.Add(GraphBuilder, InView, PassInputs, {});
		const FScreenPassTexture& ScreenTexOutput = Output.Texture;

		// Extract the result back into the composite render target
		*CompositeRenderPassPtr = GraphBuilder.ConvertToExternalTexture(ScreenTexOutput.Texture);
	}
}

void FCompositeCoreSceneViewExtension::SubscribeToPostProcessingPass(EPostProcessingPass PassId, const FSceneView& InView, FAfterPassCallbackDelegateArray& InOutPassCallbacks, bool bIsPassEnabled)
{
	if (!IsAllowedForViewFamily(*InView.Family))
	{
		return;
	}

	if (GetRenderWork().FramePasses.Contains(PassId))
	{
		InOutPassCallbacks.Add(FAfterPassCallbackDelegate::CreateRaw(this, &FCompositeCoreSceneViewExtension::PostProcessWork_RenderThread, PassId));
	}
}

bool FCompositeCoreSceneViewExtension::IsAllowedForViewFamily(const FSceneViewFamily& InViewFamily) const
{
	auto IsViewModeAllowedFn = [&InViewFamily](const TSet<EViewModeIndex>& InAllowedViewModes) -> bool
		{
			// If the allowed view modes set is empty, we allow all types.
			return InAllowedViewModes.IsEmpty() ? true : InAllowedViewModes.Contains(InViewFamily.ViewMode);
		};

	bool bIsAllowed = true;
	// SetupView clears this flag when either the view-mode or active-viewport filter rejects the family.
	bIsAllowed &= static_cast<bool>(InViewFamily.EngineShowFlags.AllowPrimitiveAlphaHoldout);

	if (IsInRenderingThread())
	{
		bIsAllowed &= IsViewModeAllowedFn(GetRenderWork().AllowedViewModes);
	}
	else if(IsInGameThread())
	{
		bIsAllowed &= IsViewModeAllowedFn(AllowedViewModes);
	}

	return bIsAllowed;
}

void FCompositeCoreSceneViewExtension::ApplyPreprocessing(FRDGBuilder& GraphBuilder, FSceneView& InView) const
{
	using namespace UE::CompositeCore;

	const TSortedMap<ResourceId, TArray<const FCompositeCorePassProxy*>>& PreprocessingPasses = GetRenderWork().PreprocessingPasses;

	if (PreprocessingPasses.IsEmpty())
	{
		return;
	}

	FPassContext PassContext = {};
	const FScopedExternalTextureMap ExternalTextures{ *this, GraphBuilder, InView };

	for (const TPair<ResourceId, TArray<const FCompositeCorePassProxy*>>& Pair : PreprocessingPasses)
	{
		const ResourceId ExternalId = Pair.Key;
		const TArray<const FCompositeCorePassProxy*>& Passes = Pair.Value;
		const FPassInput* ExternalTexture = ExternalTextures.Get().Find(ExternalId);

		if (!ensureMsgf(ExternalTexture != nullptr, TEXT("Unexpected missing external texture.")))
		{
			continue;
		}

		// Setup pass inputs
		FPassInputArray PassInputs;
		PassInputs.GetInputs().SetNum(kPostProcessMaterialInputCountMax);
		PassInputs[0] = *ExternalTexture;
		// Empty textures for the other inputs
		for (int32 Index = 1; Index < kPostProcessMaterialInputCountMax; ++Index)
		{
			PassInputs[Index].Texture = FScreenPassTexture{ GSystemTextures.GetBlackDummy(GraphBuilder) };
		}

		FPassTexture Output;
		PassContext.OutputViewRect = ExternalTexture->Texture.ViewRect;
		constexpr int32 LastMergePassIndex = -1;
		constexpr int32 RecursionLevel = 0;

		ApplyPassesRecursive(GraphBuilder, InView, PassInputs, PassInputs, PassContext, Passes, ExternalTextures, LastMergePassIndex, RecursionLevel, Output);
	}
}

UE::CompositeCore::FScopedExternalTextureMap::FScopedExternalTextureMap(const FCompositeCoreSceneViewExtension& SVE, FRDGBuilder& InGraphBuilder, const FSceneView& InView)
	: GraphBuilder{InGraphBuilder}
{
	using namespace UE::CompositeCore;

	const TArray<FExternalTexture>& ExternalInputs = SVE.GetRenderWork().ExternalInputs;

	Textures.Reserve(2 + ExternalInputs.Num());

	{
		FPassInput& Resource = Textures.Add(ResourceId::BuiltInEmpty);
		Resource.Texture = FScreenPassTexture{ GSystemTextures.GetBlackDummy(GraphBuilder) };
	}

	{
		FPassInput& Resource = Textures.Add(ResourceId::BuiltInCRP);

		const TRefCountPtr<IPooledRenderTarget>* CompositeRenderPassPtr = SVE.CustomRenderTargetPerView_RenderThread.Find(InView.GetViewKey());
		if (CompositeRenderPassPtr != nullptr)
		{
			Resource.Texture = FScreenPassTexture{ GraphBuilder.RegisterExternalTexture(*CompositeRenderPassPtr) };
			Resource.Metadata.bInvertedAlpha = true;
		}
		else
		{
			// Still provide an empty CRP target
			Resource.Texture = FScreenPassTexture{ GSystemTextures.GetBlackDummy(GraphBuilder) };
		}
	}

	RestoreToExternal.Reserve(ExternalInputs.Num());

	// Always emit an entry for every declared external slot.
	auto SetTransparentDummy = [this](FPassInput& Resource, const FResourceMetadata& Metadata)
	{
		Resource.Texture = FScreenPassTexture{ GSystemTextures.GetBlackDummy(GraphBuilder) };
		Resource.Metadata = Metadata;
		Resource.Metadata.bInvertedAlpha = false;
	};

	for (int32 Index = 0; Index < ExternalInputs.Num(); ++Index)
	{
		const FExternalTexture& ExternalInput = ExternalInputs[Index];
		const ResourceId MapIndex = MakeExternalResourceId(Index);
		FPassInput& Resource = Textures.Add(MapIndex);

		// Resolve the RHI texture at point of use so we pick up media-texture / render-target
		// updates produced after SetRenderWork_GameThread was called this frame.
		TStrongObjectPtr<UTexture> ExternalTexture = ExternalInput.Texture.Pin();
		FTextureResource* TexResource = ExternalTexture.IsValid() ? ExternalTexture->GetResource() : nullptr;

		FRHITexture* RHITexture = nullptr;
		if (TexResource != nullptr)
		{
			RHITexture = TexResource->GetTextureRHI();
		}

		if (RHITexture == nullptr)
		{
			SetTransparentDummy(Resource, ExternalInput.Metadata);
			continue;
		}

		TRefCountPtr<IPooledRenderTarget> PooledRenderTarget = CreateRenderTarget(RHITexture, ExternalInput.Metadata.DebugName);
		FRDGTextureRef TextureResource = GraphBuilder.RegisterExternalTexture(PooledRenderTarget);

		if (ensureMsgf(TextureResource, TEXT("Failed external pooled render target registration.")))
		{
			Resource.Texture = FScreenPassTexture{ TextureResource };
			Resource.Metadata = ExternalInput.Metadata;

			/**
			 * Mark for internal access and store to reset in destructor.
			 *
			 * Note: we exclude the active view family render target since it is already (and should remain) internal.
			 * In practice, this can occur when our textures are accessed by pre-processing passes during a scene capture render.
			 *
			 * Unfortunately, full AccessModeState information is not accessible outside of RDG or friend classes. See for example
			 * GraphBuilder.UseExternalAccessMode(Buffer, AccessModeState.Access, AccessModeState.Pipelines); use in FRDGResourceDumpContext.
			 */
			FRHITexture* FamilyRenderTargetRHI = InView.Family->RenderTarget ? InView.Family->RenderTarget->GetRenderTargetTexture().GetReference() : nullptr;
			if (PooledRenderTarget->GetRHI() != FamilyRenderTargetRHI)
			{
				GraphBuilder.UseInternalAccessMode(TextureResource);

				RestoreToExternal.Add(TextureResource);
			}
		}
		else
		{
			SetTransparentDummy(Resource, ExternalInput.Metadata);
		}
	}
}

UE::CompositeCore::FScopedExternalTextureMap::~FScopedExternalTextureMap()
{
	for (FRDGViewableResource* TextureResource : RestoreToExternal)
	{
		GraphBuilder.UseExternalAccessMode(TextureResource, ERHIAccess::SRVMask);
	}
}

bool FCompositeCoreSceneViewExtension::ApplyPassesRecursive(
	FRDGBuilder& GraphBuilder,
	const FSceneView& InView,
	const UE::CompositeCore::FPassInputArray& Inputs,
	const UE::CompositeCore::FPassInputArray& OriginalInputs,
	UE::CompositeCore::FPassContext& PassContext,
	const TArray<const FCompositeCorePassProxy*> InPasses,
	const UE::CompositeCore::FScopedExternalTextureMap& InExternalTextures,
	const int32 LastMergePassIndex,
	const int32 RecursionLevel,
	UE::CompositeCore::FPassTexture& Output) const
{
	using namespace UE::CompositeCore;

	if (InPasses.IsEmpty())
	{
		return false;
	}

	// Default pass inputs
	FPassInputArray BasePassInputs = Inputs;
	const TSortedMap<const FCompositeCorePassProxy*, const UE::CompositeCore::FPassTexture>& CachedOutputs = CachedOutputsPerView_RenderThread.FindOrAdd(InView.GetViewKey());
	
	// Iterate over all passes
	for (int32 PassIndex = 0; PassIndex < InPasses.Num(); ++PassIndex)
	{
		const FCompositeCorePassProxy* Pass = InPasses[PassIndex];
		
		// Last merge pass is used to output values compatible with scene color
		bool bIsLastMergePass = false;
		// Last pass writes to the viewport render target
		bool bIsLastPass = false;
		
		if (RecursionLevel == 0)
		{
			bIsLastMergePass = (PassIndex == LastMergePassIndex);
			bIsLastPass = (PassIndex == InPasses.Num() - 1);
		}

		// Update inputs for the current pass
		FPassInputArray PassInputs = BasePassInputs;

		// Iterate over declared pass inputs
		for (int32 InputIndex = 0; InputIndex < Pass->GetNumDeclaredInputs(); ++InputIndex)
		{
			const FPassInputDecl& DeclaredInput = Pass->GetDeclaredInput(InputIndex);

			// Input is the output result of a child (sub)pass
			if (DeclaredInput.IsType<const FCompositeCorePassProxy*>())
			{
				const FCompositeCorePassProxy* ChildPassProxy = DeclaredInput.Get<const FCompositeCorePassProxy*>();
				UE::CompositeCore::FPassTexture ChildPassOutput;

				// We first check if the input proxy has already produced an output (and let RDG do the rest!)
				if (const UE::CompositeCore::FPassTexture* PreProducedOutput = CachedOutputs.Find(ChildPassProxy))
				{
					PassInputs[InputIndex] = *PreProducedOutput;
				}
				else if(ApplyPassesRecursive(GraphBuilder, InView, PassInputs, OriginalInputs, PassContext, { ChildPassProxy }, InExternalTextures, INDEX_NONE, RecursionLevel + 1, ChildPassOutput))
				{
					PassInputs[InputIndex] = ChildPassOutput;
				}
			}
			// If an external texture resource is expected, connect the current input index with the resolved identifier
			else if (DeclaredInput.IsType<FPassExternalResourceDesc>())
			{
				const FPassExternalResourceDesc& Desc = DeclaredInput.Get<FPassExternalResourceDesc>();
				const ResourceId DeclaredExternalId = Desc.Id;

				const FPassInput* SliceInput = InExternalTextures.Get().Find(DeclaredExternalId);
				if (ensureMsgf(SliceInput, TEXT("Invalid external input: %u"), UE::CompositeCore::ToIndex(DeclaredExternalId)))
				{
					PassInputs[InputIndex] = *SliceInput;
				}
				else
				{
					UE_CALL_ONCE([]()
						{
							UE_LOGF(LogCompositeCore, Warning, "Compositing may be incorrectly running inside a scene capture render, and trying to use render targets which have not yet been updated. Consider disabling AllowPrimitiveAlphaHoldout on your scene capture(s).")
						});
				}
			}
			// If an internal texture resource is expected, connect the current input index with either the original or current bass pass input.
			else if (DeclaredInput.IsType<FPassInternalResourceDesc>())
			{
				const FPassInternalResourceDesc& Desc = DeclaredInput.Get<FPassInternalResourceDesc>();
				const int32 DeclaredInputIndex = Desc.Index;

				if (Desc.bOriginalCopyBeforePasses)
				{
					if (ensureMsgf(OriginalInputs.IsValidIndex(DeclaredInputIndex), TEXT("Invalid internal input: %d"), DeclaredInputIndex))
					{
						PassInputs[InputIndex] = OriginalInputs[DeclaredInputIndex];
					}
				}
				else
				{
					if (ensureMsgf(BasePassInputs.IsValidIndex(DeclaredInputIndex), TEXT("Invalid internal input: %d"), DeclaredInputIndex))
					{
						PassInputs[InputIndex] = BasePassInputs[DeclaredInputIndex];
					}
				}
			}
			else
			{
				checkNoEntry();
			}
		}

		const TOptional<UE::CompositeCore::ResourceId>& OverrrideOutput = Pass->GetDeclaredPrimaryOutputOverride();

		if (OverrrideOutput.IsSet())
		{
			const FPassInput* ExternalOutputOverride = InExternalTextures.Get().Find(OverrrideOutput.GetValue());
			if (ensureMsgf(ExternalOutputOverride != nullptr, TEXT("Unexpected missing external render target override as output.")))
			{
				PassInputs.OverrideOutput = FScreenPassRenderTarget(ExternalOutputOverride->Texture, ERenderTargetLoadAction::ENoAction);

				// Update view rect to match external texture
				PassContext.OutputViewRect = ExternalOutputOverride->Texture.ViewRect;
			}
		}
		else
		{
			if (bIsLastMergePass)
			{
				// Merge pass proxies will handle alpha inversion, lens distortion and color encodings when writing back to scene color
				PassContext.bOutputSceneColor = true;
			}

			if (!bIsLastPass)
			{
				// Only apply the output override on the last pass
				PassInputs.OverrideOutput = FScreenPassRenderTarget{};
			}
		}

		// Register pass and update output
		Output = Pass->Add(GraphBuilder, InView, PassInputs, PassContext);
		
		// Cache the pass output per view
		CachedOutputsPerView_RenderThread.FindOrAdd(InView.GetViewKey()).Add(Pass, Output);

		BasePassInputs[0] = Output;
	}

	return true;
}

FScreenPassTexture FCompositeCoreSceneViewExtension::PostProcessWork_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& InView, const FPostProcessMaterialInputs& Inputs, ISceneViewExtension::EPostProcessingPass InLocation)
{
	using namespace UE::CompositeCore;

	const FScopedExternalTextureMap ExternalTextures{ *this, GraphBuilder, InView };

	const TArray<const FCompositeCorePassProxy*>* PassesPtr = GetRenderWork().FramePasses.Find(InLocation);
	if ((PassesPtr == nullptr) || PassesPtr->IsEmpty())
	{
		return Inputs.ReturnUntouchedSceneColorForPostProcessing(GraphBuilder);
	}
	const TArray<const FCompositeCorePassProxy*>& Passes = *PassesPtr;

	FPassContext PassContext;
	PassContext.SceneTextures = Inputs.SceneTextures;
	PassContext.OutputViewRect = Inputs.GetInput(EPostProcessMaterialInput::SceneColor).ViewRect;
	PassContext.Location = InLocation;
	PassContext.bOutputSceneColor = false;

	FPassInputArray ResolvedInputs(GraphBuilder, InView, Inputs, InLocation);
	FPassTexture Output;

	const int32 LastMergePassIndex = Passes.FindLastByPredicate([](const FCompositeCorePassProxy* InPass)
		{
			return InPass ? (InPass->GetTypeName() == GetMergePassPassName()) : false;
		});
	constexpr int32 RecursionLevel = 0;
	
	if (ApplyPassesRecursive(GraphBuilder, InView, ResolvedInputs, ResolvedInputs, PassContext, Passes, ExternalTextures, LastMergePassIndex, RecursionLevel, Output))
	{
		return MoveTemp(Output.Texture);
	}
	else
	{
		return Inputs.ReturnUntouchedSceneColorForPostProcessing(GraphBuilder);
	}
}

void FCompositeCoreSceneViewExtension::PostRenderViewFamily_RenderThread(FRDGBuilder& GraphBuilder, FSceneViewFamily& InViewFamily)
{

}

void FCompositeCoreSceneViewExtension::PostRenderView_RenderThread(FRDGBuilder& GraphBuilder, FSceneView& InView)
{
	CustomRenderTargetPerView_RenderThread.Remove(InView.GetViewKey());
}
