// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneViewExtension.h"

#include "Containers/ContainersFwd.h"
#include "RenderGraphFwd.h"
#include "RendererInterface.h"

#include "CompositeCoreSettings.h"
#include "CompositeCoreTypes.h"
#include "Passes/CompositeCorePassProxy.h"

class FViewInfo;
class FCompositeCoreSceneViewExtension;
struct FScreenPassRenderTarget;
struct FScreenPassTexture;
struct FScreenPassTextureSlice;
class UMaterialInterface;
class UTexture;

enum class EPostProcessMaterialInput : uint32;

namespace UE
{
	namespace CompositeCore
	{
		/** Collection of external textures scoped to internal RDG access. */
		struct FScopedExternalTextureMap
		{
			UE_NONCOPYABLE(FScopedExternalTextureMap);

			/** Constructor */
			FScopedExternalTextureMap(const FCompositeCoreSceneViewExtension& SVE, FRDGBuilder& InGraphBuilder, const FSceneView& InView);
			
			/** Destructor */
			~FScopedExternalTextureMap();

			/** Get the map of ResourceId to pass input textures. */
			const TSortedMap<UE::CompositeCore::ResourceId, UE::CompositeCore::FPassInput>& Get() const
			{
				return Textures;
			}

		private:
			/** Host graph builder */
			FRDGBuilder& GraphBuilder;
			/** Map of pass input external textures. */
			TSortedMap<UE::CompositeCore::ResourceId, UE::CompositeCore::FPassInput> Textures;
			/** List of textures marked for internal access for external restoration upon destruction. */
			TArray<FRDGViewableResource*, SceneRenderingAllocator> RestoreToExternal;
		};
	}
}

class FCompositeCoreSceneViewExtension : public FWorldSceneViewExtension
{
public:
	FCompositeCoreSceneViewExtension(const FAutoRegister& AutoReg, UWorld* InWorld);
	~FCompositeCoreSceneViewExtension();

	/** Register primitives for compositing. */
	void RegisterPrimitives_GameThread(const TArray<UPrimitiveComponent*>& InPrimitiveComponents, ECompositeCoreHoldoutManagement HoldoutManagement = ECompositeCoreHoldoutManagement::Automatic);

	/** Unregister primitives for compositing. */
	void UnregisterPrimitives_GameThread(const TArray<UPrimitiveComponent*>& InPrimitiveComponents, ECompositeCoreHoldoutManagement HoldoutManagement = ECompositeCoreHoldoutManagement::Automatic);

	/** Set frame render work to be enqueued on the render thread. */
	void SetRenderWork_GameThread(UE::CompositeCore::FRenderWork&& InWork);

	/** Reset frame render work. */
	void ResetRenderWork_GameThread();

	/** Set options for built-in composite custom render pass. */
	void SetBuiltInRenderPassOptions_GameThread(const UE::CompositeCore::FBuiltInRenderPassOptions& InOptions);
	
	/** Reset default options for built-in composite custom render pass. */
	void ResetBuiltInRenderPassOptions_GameThread();

	/** Called by the custom render pass to store its view render target for this frame. */
	void CollectCustomRenderTarget(uint32 InViewId, const TRefCountPtr<IPooledRenderTarget>& InRenderTarget);

	//~ Begin ISceneViewExtension Interface
	virtual int32 GetPriority() const override;
	virtual void SetupViewFamily(FSceneViewFamily& InViewFamily) override;
	virtual void SetupView(FSceneViewFamily& InViewFamily, FSceneView& InView) override;
	virtual void BeginRenderViewFamily(FSceneViewFamily& InViewFamily) override;
	virtual void PreRenderViewFamily_RenderThread(FRDGBuilder& GraphBuilder, FSceneViewFamily& InViewFamily) override;
	virtual void PreRenderView_RenderThread(FRDGBuilder& GraphBuilder, FSceneView& InView) override;
	virtual void PrePostProcessPass_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& InView, const FPostProcessingInputs& Inputs) override;
	virtual void SubscribeToPostProcessingPass(EPostProcessingPass PassId, const FSceneView& InView, FAfterPassCallbackDelegateArray& InOutPassCallbacks, bool bIsPassEnabled) override;
	virtual void PostRenderViewFamily_RenderThread(FRDGBuilder& GraphBuilder, FSceneViewFamily& InViewFamily) override;
	virtual void PostRenderView_RenderThread(FRDGBuilder& GraphBuilder, FSceneView& InView) override;
	//~ End ISceneViewExtension Interface

protected:
	//~ Begin ISceneViewExtension Interface
	virtual bool IsActiveThisFrame_Internal(const FSceneViewExtensionContext& Context) const override;
	//~ End ISceneViewExtension Interface

private:
	/** Getter for the frame render work, either from the actor or default. */
	const UE::CompositeCore::FRenderWork& GetRenderWork() const;

	/** Check if render work is allowed on the view family. */
	bool IsAllowedForViewFamily(const FSceneViewFamily& InViewFamily) const;

	/** Once-per frame external resource processing passes. */
	void ApplyPreprocessing(FRDGBuilder& GraphBuilder, FSceneView& InView) const;

	/** Recursive pass application. */
	bool ApplyPassesRecursive(
		FRDGBuilder& GraphBuilder,
		const FSceneView& InView,
		const UE::CompositeCore::FPassInputArray& Inputs,
		const UE::CompositeCore::FPassInputArray& OriginalInputs,
		UE::CompositeCore::FPassContext& PassContext,
		const TArray<const FCompositeCorePassProxy*> InPasses,
		const UE::CompositeCore::FScopedExternalTextureMap& InExternalTextures,
		const int32 LastMergePassIndex,
		const int32 RecursionLevel,
		UE::CompositeCore::FPassTexture& Output
	) const;

	/** Callback for processing passes. */
	FScreenPassTexture PostProcessWork_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& InView,
		const FPostProcessMaterialInputs& Inputs, ISceneViewExtension::EPostProcessingPass InLocation);

	/** Collection of primitives to render as a custom render pass and composite after post-processing. .*/
	TSet<TWeakObjectPtr<UPrimitiveComponent>> CompositePrimitives;

	/** Custom render pass render targets for each active view .*/
	TSortedMap<uint32, TRefCountPtr<IPooledRenderTarget>> CustomRenderTargetPerView_RenderThread;

	/** Custom render work. */
	TOptional<UE::CompositeCore::FRenderWork> RenderWork_RenderThread;

	/** Built-in composite render pass options.*/
	TOptional<UE::CompositeCore::FBuiltInRenderPassOptions> BuiltInRenderPassOptions;

	/** Used to keep the SVE active if any custom render work is present.*/
	bool bHasCustomRenderWork = false;

	/** Optional main render mode override */
	TOptional<ESceneCaptureSource> MainRenderMode;

	/** View modes for which the compositing is allowed. */
	TSet<EViewModeIndex> AllowedViewModes;

	/** Game-thread copy of the active viewport filter, consulted by SetupView. */
	TOptional<UE::CompositeCore::FActiveViewportFilter> ActiveViewportFilter;

	/** List of scene captures to render-update this frame. */
	TArray<TWeakObjectPtr<USceneCaptureComponent2D>> SceneCapturesUpdateQueue;

	/**
	* Map of cached proxy output textures per view (post-processing only currently).
	* 
	* Passes can then refer to a previous proxy as an input type, and if that proxy has
	* already been executed its output will be used directly without needing to duplicate work.
	* This is possible since RDG automatically knows to connect & preserve the texture resource
	* when it is later used as an input of a subsequent pass.
	*/
	mutable TSortedMap<uint32, TSortedMap<const FCompositeCorePassProxy*, const UE::CompositeCore::FPassTexture>> CachedOutputsPerView_RenderThread;

	friend struct UE::CompositeCore::FScopedExternalTextureMap;
};

