// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositeCoreSettings.h"
#include "CompositeCoreSubsystem.h"

#include "Components/PrimitiveComponent.h"
#include "Misc/ScopeLock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CompositeCoreSettings)

namespace UE::CompositeCore
{
	static bool GRegisterPrimitivesOnTick = true;

	bool IsRegisterPrimitivesOnTickEnabled()
	{
		return GRegisterPrimitivesOnTick;
	}
}
static FAutoConsoleVariableRef CVarRefreshPrimitivesOnTick(
	TEXT("CompositeCore.RefreshPrimitivesOnTick"),
	UE::CompositeCore::GRegisterPrimitivesOnTick,
	TEXT("Refresh primitive registration on every tick. Enabled by default for use cases such as dynamic text 3D.\n")
	TEXT("Read-only and to be set in a config file (requires restart)."),
	ECVF_ReadOnly);

UCompositeCorePluginSettings::UCompositeCorePluginSettings()
	: bApplyPreExposure(0)
	, DisabledPrimitiveClasses(
		{
			FSoftClassPath("/Script/Engine.CapsuleComponent"),
			FSoftClassPath("/Script/Engine.BillboardComponent"),
			FSoftClassPath("/Script/Engine.ArrowComponent"),
			FSoftClassPath("/Script/Engine.DrawFrustumComponent"),
			FSoftClassPath("/Script/Engine.LineBatchComponent"),
			FSoftClassPath("/Script/Engine.DrawSphereComponent")
		}
	)
	, AllowedComponentClasses(
		{
			FSoftClassPath("/Script/Text3D.Text3DComponent"),
		}
	)
	, SceneViewExtensionPriority(100 + 1 /* OPENCOLORIO_SCENE_VIEW_EXTENSION_PRIORITY + 1 */)
	, CachedDisabledPrimitiveClasses()
	, bIsPrimitiveCacheDirty(true)
	, CachedAllowedComponentClasses()
	, bIsComponentCacheDirty(true)
{ }


FName UCompositeCorePluginSettings::GetCategoryName() const
{
	return TEXT("Plugins");
}

#if WITH_EDITOR
FText UCompositeCorePluginSettings::GetSectionText() const
{
	return NSLOCTEXT("CompositeCoreSettings", "CompositeCoreSettingsSection", "Composite Core");
}

FName UCompositeCorePluginSettings::GetSectionName() const
{
	return TEXT("Composite Core");
}

void UCompositeCorePluginSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(ThisClass, DisabledPrimitiveClasses))
	{
		bIsPrimitiveCacheDirty = true;
	}
	else if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(ThisClass, AllowedComponentClasses))
	{
		bIsComponentCacheDirty = true;
	}

	if (PropertyChangedEvent.Property)
	{
		ExportValuesToConsoleVariables(PropertyChangedEvent.Property);
	}
}
#endif

void UCompositeCorePluginSettings::PostInitProperties()
{
	Super::PostInitProperties();

#if WITH_EDITOR
	if (IsTemplate())
	{
		ImportConsoleVariableValues();
	}
#endif // #if WITH_EDITOR
}

bool UCompositeCorePluginSettings::IsAllowedPrimitiveClass(const UPrimitiveComponent* InPrimitiveComponent) const
{
	if (!InPrimitiveComponent)
	{
		return false;
	}

	// Note: Should always be called from the game thread, but we lock the mutable cache just in case.
	FScopeLock Lock(&CriticalSection);

	if (bIsPrimitiveCacheDirty)
	{
		CacheDisabledPrimitiveClasses();
	}

	for (const UClass* ObjectClass : CachedDisabledPrimitiveClasses)
	{
		if (InPrimitiveComponent->IsA(ObjectClass))
		{
			return false;
		}
	}

	return true;
}

bool UCompositeCorePluginSettings::IsAllowedComponentClass(const USceneComponent* InComponent) const
{
	if (!InComponent)
	{
		return false;
	}

	{
		// Note: Should always be called from the game thread, but we lock the mutable cache just in case.
		FScopeLock Lock(&CriticalSection);
		
		if (bIsComponentCacheDirty)
		{
			CacheAllowedComponentClasses();
		}

		for (const FSoftClassPath& ObjectClass : AllowedComponentClasses)
		{
			if (const UClass* AllowedClass = ObjectClass.TryLoadClass<UObject>())
			{
				if (InComponent->IsA(AllowedClass))
				{
					return true;
				}
			}
		}
	}

	if (const UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(InComponent))
	{
		return IsAllowedPrimitiveClass(PrimitiveComponent);
	}

	return false;
}

void UCompositeCorePluginSettings::CacheDisabledPrimitiveClasses() const
{
	CachedDisabledPrimitiveClasses.Reset(0);

	for (const FSoftClassPath& ObjectClass : DisabledPrimitiveClasses)
	{
		if (const UClass* DisabledClass = ObjectClass.TryLoadClass<UObject>())
		{
			CachedDisabledPrimitiveClasses.Add(DisabledClass);
		}
	}

	bIsPrimitiveCacheDirty = false;
}

void UCompositeCorePluginSettings::CacheAllowedComponentClasses() const
{
	CachedAllowedComponentClasses.Reset(0);

	for (const FSoftClassPath& ObjectClass : AllowedComponentClasses)
	{
		if (const UClass* AllowedClass = ObjectClass.TryLoadClass<UObject>())
		{
			CachedAllowedComponentClasses.Add(AllowedClass);
		}
	}

	bIsComponentCacheDirty = false;
}
