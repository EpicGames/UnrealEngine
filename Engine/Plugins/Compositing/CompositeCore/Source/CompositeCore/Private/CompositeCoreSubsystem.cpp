// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositeCoreSubsystem.h"

#include "CompositeCoreModule.h"
#include "CompositeCoreSceneViewExtension.h"

#include "Engine/RendererSettings.h"
#include "HAL/IConsoleManager.h"

#if WITH_EDITOR
#include "Framework/Notifications/NotificationManager.h"
#include "HAL/PlatformFileManager.h"
#include "ISettingsEditorModule.h"
#include "ISourceControlModule.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"
#include "SourceControlHelpers.h"
#include "Widgets/Notifications/SNotificationList.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(CompositeCoreSubsystem)

#define LOCTEXT_NAMESPACE "CompositeCoreSubsystem"

namespace
{
#if WITH_EDITOR
	void UpdateDependentPropertyInConfigFile(URendererSettings* RendererSettings, FProperty* RendererProperty)
	{
		const FString RelativePath = RendererSettings->GetDefaultConfigFilename();
		const FString FullPath = FPaths::ConvertRelativePathToFull(RelativePath);

		const bool bIsReadOnly = FPlatformFileManager::Get().GetPlatformFile().IsReadOnly(*FullPath);
		bool bCheckedOutFromSourceControl = false;

		if (bIsReadOnly)
		{
			// Prefer to check out via the editor's source-control provider so the user sees the file
			// appear in their pending changelist. Falls through to a raw read-only toggle on failure
			// (provider disabled, not connected, or file untracked).
			if (ISourceControlModule::Get().IsEnabled())
			{
				bCheckedOutFromSourceControl = USourceControlHelpers::CheckOutOrAddFile(FullPath);
				if (!bCheckedOutFromSourceControl)
				{
					UE_LOGF(LogCompositeCore, Warning, "Could not check out '%ls' via source control: %ls", *FullPath, *USourceControlHelpers::LastErrorMsg().ToString());
				}
			}

			if (!bCheckedOutFromSourceControl)
			{
				FPlatformFileManager::Get().GetPlatformFile().SetReadOnly(*FullPath, false);
			}
		}

		RendererSettings->UpdateSinglePropertyInConfigFile(RendererProperty, RendererSettings->GetDefaultConfigFilename());

		// If we only made the file writable on the file system (no source-control checkout),
		// restore the read-only bit so a future sync from source control isn't blocked.
		if (bIsReadOnly && !bCheckedOutFromSourceControl)
		{
			FPlatformFileManager::Get().GetPlatformFile().SetReadOnly(*FullPath, true);
		}
	}
#endif
}

UCompositeCoreSubsystem::UCompositeCoreSubsystem()
{
}

void UCompositeCoreSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	UWorld* World = GetWorld();

	if (IsValid(World))
	{
		CompositeCoreViewExtension = FSceneViewExtensions::NewExtension<FCompositeCoreSceneViewExtension>(World);
	}
}

void UCompositeCoreSubsystem::Deinitialize()
{
	CompositeCoreViewExtension.Reset();

	Super::Deinitialize();
}

void UCompositeCoreSubsystem::Tick(float DeltaTime)
{
	Super::Tick(DeltaTime);
}

void UCompositeCoreSubsystem::RegisterPrimitive(UPrimitiveComponent* InPrimitiveComponent, ECompositeCoreHoldoutManagement HoldoutManagement)
{
	TArray<UPrimitiveComponent*> PrimitiveComponents = { InPrimitiveComponent };

	RegisterPrimitives(PrimitiveComponents, HoldoutManagement);
}

void UCompositeCoreSubsystem::RegisterPrimitives(const TArray<UPrimitiveComponent*>& InPrimitiveComponents, ECompositeCoreHoldoutManagement HoldoutManagement)
{
	const bool bValidSettings = ValidateProjectSettings();

	if (bValidSettings && CompositeCoreViewExtension.IsValid())
	{
		CompositeCoreViewExtension->RegisterPrimitives_GameThread(InPrimitiveComponents, HoldoutManagement);
	}
}

void UCompositeCoreSubsystem::UnregisterPrimitive(UPrimitiveComponent* InPrimitiveComponent, ECompositeCoreHoldoutManagement HoldoutManagement)
{
	TArray<UPrimitiveComponent*> PrimitiveComponents = { InPrimitiveComponent };

	UnregisterPrimitives(PrimitiveComponents, HoldoutManagement);
}

void UCompositeCoreSubsystem::UnregisterPrimitives(const TArray<UPrimitiveComponent*>& InPrimitiveComponents, ECompositeCoreHoldoutManagement HoldoutManagement)
{
	if (CompositeCoreViewExtension.IsValid())
	{
		CompositeCoreViewExtension->UnregisterPrimitives_GameThread(InPrimitiveComponents, HoldoutManagement);
	}
}

void UCompositeCoreSubsystem::SetRenderWork(UE::CompositeCore::FRenderWork&& InWork)
{
	if (CompositeCoreViewExtension.IsValid())
	{
		CompositeCoreViewExtension->SetRenderWork_GameThread(MoveTemp(InWork));
	}
}

void UCompositeCoreSubsystem::ResetRenderWork()
{
	if (CompositeCoreViewExtension.IsValid())
	{
		CompositeCoreViewExtension->ResetRenderWork_GameThread();
	}
}

void UCompositeCoreSubsystem::SetBuiltInRenderPassOptions(const UE::CompositeCore::FBuiltInRenderPassOptions& InOptions)
{
	if (CompositeCoreViewExtension.IsValid())
	{
		CompositeCoreViewExtension->SetBuiltInRenderPassOptions_GameThread(InOptions);
	}
}

void UCompositeCoreSubsystem::ResetBuiltInRenderPassOptions()
{
	if (CompositeCoreViewExtension.IsValid())
	{
		CompositeCoreViewExtension->ResetBuiltInRenderPassOptions_GameThread();
	}
}

bool UCompositeCoreSubsystem::IsProjectSettingsValid()
{
	static const auto CVarAlphaOutput = IConsoleManager::Get().FindConsoleVariable(TEXT("r.PostProcessing.PropagateAlpha"));
	static const auto CVarSupportPrimitiveAlphaHoldout = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Deferred.SupportPrimitiveAlphaHoldout"));

	const bool bAlphaOutput = CVarAlphaOutput ? CVarAlphaOutput->GetBool() : false;
	const bool bSupportPrimitiveAlphaHoldout = CVarSupportPrimitiveAlphaHoldout ? CVarSupportPrimitiveAlphaHoldout->GetBool() : false;

	return bAlphaOutput && bSupportPrimitiveAlphaHoldout;
}

bool UCompositeCoreSubsystem::ValidateProjectSettings()
{
	const bool bValidSettings = IsProjectSettingsValid();

	if (!bValidSettings )
	{
#if WITH_EDITOR
		// We inform the user and offer them the option to activate project settings.
		UE_CALL_ONCE([&]{ PrimitiveHoldoutSettingsNotification(); });
#else
		UE_CALL_ONCE([&]{ UE_LOGF(LogCompositeCore, Warning, "Both \"Alpha Output\" and \"Support Primitive Alpha Holdout\" project settings must be enabled for holdout composite."); });
#endif
	}

	return bValidSettings;
}

#if WITH_EDITOR
void UCompositeCoreSubsystem::PrimitiveHoldoutSettingsNotification()
{
	URendererSettings* RendererSettings = GetMutableDefault<URendererSettings>();
	check(RendererSettings);

	const bool bAlphaOutputMissing = !RendererSettings->bEnableAlphaChannelInPostProcessing;
	const bool bPrimitiveHoldoutMissing = !RendererSettings->bDeferredSupportPrimitiveAlphaHoldout;

	if (!bAlphaOutputMissing && !bPrimitiveHoldoutMissing)
	{
		UE_LOGF(LogCompositeCore, Warning, "\"Alpha Output\" and \"Support Primitive Alpha Holdout\" project settings are valid, but one or more console variables have been disabled.");
		return;
	}

	const FText AlphaOutputSettingOption = LOCTEXT("HoldoutSetting_AlphaOutput", "\n- Alpha Output");
	const FText PrimitiveHoldoutSettingOption = LOCTEXT("HoldoutSetting_PrimitiveHoldout", "\n- Support Primitive Alpha Holdout");
	const FText HoldoutText = FText::Format(LOCTEXT("HoldoutSettingPrompt", "The following project setting(s) must be enabled for holdout composite:{0}{1}\n\nWarning: update can add renderer performance costs."), bAlphaOutputMissing ? AlphaOutputSettingOption : FText::GetEmpty(), bPrimitiveHoldoutMissing ? PrimitiveHoldoutSettingOption : FText::GetEmpty());
	const FText HoldoutConfirmText = LOCTEXT("HoldoutSettingConfirm", "Enable (DefaultEngine.ini)");
	const FText HoldoutCancelText = LOCTEXT("HoldoutSettingCancel", "Not Now");

	/** Utility functions for notifications */
	struct FSuppressDialogOptions
	{
		static bool ShouldSuppressModal()
		{
			bool bSuppressNotification = false;
			GConfig->GetBool(TEXT("CompositeCore"), TEXT("SuppressCompositeCorePromptNotification"), bSuppressNotification, GEditorPerProjectIni);
			return bSuppressNotification;
		}

		static ECheckBoxState GetDontAskAgainCheckBoxState()
		{
			return ShouldSuppressModal() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}

		static void OnDontAskAgainCheckBoxStateChanged(ECheckBoxState NewState)
		{
			// If the user selects to not show this again, set that in the config so we know about it in between sessions
			const bool bSuppressNotification = (NewState == ECheckBoxState::Checked);
			GConfig->SetBool(TEXT("CompositeCore"), TEXT("SuppressCompositeCorePromptNotification"), bSuppressNotification, GEditorPerProjectIni);
		}
	};

	// If the user has specified to suppress this pop up, then just early out and exit
	if (FSuppressDialogOptions::ShouldSuppressModal())
	{
		return;
	}

	FSimpleDelegate OnConfirmDelegate = FSimpleDelegate::CreateLambda(
		[WeakThis = MakeWeakObjectPtr(this), RendererSettings]()
		{
			if (IsValid(RendererSettings))
			{
				if (!RendererSettings->bDeferredSupportPrimitiveAlphaHoldout)
				{
					FProperty* Property = RendererSettings->GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(URendererSettings, bDeferredSupportPrimitiveAlphaHoldout));
					RendererSettings->PreEditChange(Property);

					RendererSettings->bDeferredSupportPrimitiveAlphaHoldout = true;

					FPropertyChangedEvent PropertyChangedEvent(Property, EPropertyChangeType::ValueSet, { RendererSettings });
					RendererSettings->PostEditChangeProperty(PropertyChangedEvent);
					UpdateDependentPropertyInConfigFile(RendererSettings, Property);

					// SupportPrimitiveAlphaHoldout requires shader recompilation, ask for a restart.
					FModuleManager::GetModuleChecked<ISettingsEditorModule>("SettingsEditor").OnApplicationRestartRequired();
				}

				if (!RendererSettings->bEnableAlphaChannelInPostProcessing)
				{
					FProperty* Property = RendererSettings->GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(URendererSettings, bEnableAlphaChannelInPostProcessing));
					RendererSettings->PreEditChange(Property);

					RendererSettings->bEnableAlphaChannelInPostProcessing = true;

					FPropertyChangedEvent PropertyChangedEvent(Property, EPropertyChangeType::ValueSet, { RendererSettings });
					RendererSettings->PostEditChangeProperty(PropertyChangedEvent);
					UpdateDependentPropertyInConfigFile(RendererSettings, Property);
				}
			}

			TStrongObjectPtr<UCompositeCoreSubsystem> Subsystem = WeakThis.Pin();
			if (Subsystem.IsValid())
			{
				TSharedPtr<SNotificationItem> NotificationItem = Subsystem->HoldoutNotificationItem.Pin();
				if (NotificationItem.IsValid())
				{
					NotificationItem->SetCompletionState(SNotificationItem::CS_Success);
					NotificationItem->ExpireAndFadeout();
				}

				Subsystem->HoldoutNotificationItem.Reset();
			}
		}
	);

	FSimpleDelegate OnCancelDelegate = FSimpleDelegate::CreateLambda(
		[WeakThis = MakeWeakObjectPtr(this)]()
		{
			TStrongObjectPtr<UCompositeCoreSubsystem> Subsystem = WeakThis.Pin();
			if (Subsystem.IsValid())
			{
				TSharedPtr<SNotificationItem> NotificationItem = Subsystem->HoldoutNotificationItem.Pin();
				if (NotificationItem.IsValid())
				{
					NotificationItem->SetCompletionState(SNotificationItem::CS_None);
					NotificationItem->ExpireAndFadeout();
				}

				Subsystem->HoldoutNotificationItem.Reset();
			}
		}
	);

	FNotificationInfo Info(HoldoutText);
	Info.bFireAndForget = false;
	Info.bUseLargeFont = false;
	Info.bUseThrobber = false;
	Info.bUseSuccessFailIcons = false;
	Info.ButtonDetails.Add(FNotificationButtonInfo(HoldoutConfirmText, FText(), OnConfirmDelegate));
	Info.ButtonDetails.Add(FNotificationButtonInfo(HoldoutCancelText, FText(), OnCancelDelegate));

	// Add a "Don't show this again" option
	Info.CheckBoxState = TAttribute<ECheckBoxState>::Create(&FSuppressDialogOptions::GetDontAskAgainCheckBoxState);
	Info.CheckBoxStateChanged = FOnCheckStateChanged::CreateStatic(&FSuppressDialogOptions::OnDontAskAgainCheckBoxStateChanged);
	Info.CheckBoxText = LOCTEXT("DontShowThisAgainCheckBoxMessage", "Don't show this again");

	if (HoldoutNotificationItem.IsValid())
	{
		HoldoutNotificationItem.Pin()->ExpireAndFadeout();
		HoldoutNotificationItem.Reset();
	}

	HoldoutNotificationItem = FSlateNotificationManager::Get().AddNotification(Info);

	if (HoldoutNotificationItem.IsValid())
	{
		HoldoutNotificationItem.Pin()->SetCompletionState(SNotificationItem::CS_Pending);
	}
}
#endif

#undef LOCTEXT_NAMESPACE

