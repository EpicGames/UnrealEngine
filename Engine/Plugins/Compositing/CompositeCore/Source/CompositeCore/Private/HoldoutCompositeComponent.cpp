// Copyright Epic Games, Inc. All Rights Reserved.

#include "HoldoutCompositeComponent.h"

#include "Components/PrimitiveComponent.h"
#include "Engine/Engine.h"
#include "CompositeCoreModule.h"
#include "CompositeCoreSettings.h"
#include "CompositeCoreSubsystem.h"

#if WITH_EDITOR
#include "CoreGlobals.h"
#include "Misc/ConfigCacheIni.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(HoldoutCompositeComponent)

#define LOCTEXT_NAMESPACE "CompositeCore"

namespace UE
{
	namespace CompositeCore
	{
		namespace Private
		{
			// Find list of primitives from the parent component and its children. We need to traverse children to support objects such as UText3DComponent.
			static TArray<UPrimitiveComponent*> FindPrimitiveComponents(USceneComponent* InParentComponent)
			{
				const UCompositeCorePluginSettings* PluginSettings = GetDefault<UCompositeCorePluginSettings>();
				check(PluginSettings);

				TArray<UPrimitiveComponent*> OutPrimitiveComponents;

				if (IsValid(InParentComponent))
				{
					{
						UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(InParentComponent);

						if (PluginSettings->IsAllowedPrimitiveClass(PrimitiveComponent))
						{
							OutPrimitiveComponents.Add(PrimitiveComponent);
						}
					}

					TArray<USceneComponent*> ParentChildComponents;
					InParentComponent->GetChildrenComponents(true /*bIncludeAllDescendants*/, ParentChildComponents);

					for (USceneComponent* ParentChild : ParentChildComponents)
					{
						UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(ParentChild);

						if (PluginSettings->IsAllowedPrimitiveClass(PrimitiveComponent))
						{
							OutPrimitiveComponents.Add(PrimitiveComponent);
						}
					}
				}

				return OutPrimitiveComponents;
			}
		}
	}
}

FHoldoutCompositeComponentDelegate UHoldoutCompositeComponent::OnComponentCreatedDelegate;

UHoldoutCompositeComponent::UHoldoutCompositeComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	if (UE::CompositeCore::IsRegisterPrimitivesOnTickEnabled())
	{
		PrimaryComponentTick.TickGroup = TG_PostUpdateWork;
		PrimaryComponentTick.bStartWithTickEnabled = true;
		PrimaryComponentTick.bCanEverTick = true;
		PrimaryComponentTick.bTickEvenWhenPaused = true;
		bTickInEditor = true;
		bAutoActivate = true;
	}
}

void UHoldoutCompositeComponent::BeginDestroy()
{
	Super::BeginDestroy();

	// Kept for safety, but redundant since the scene view extension will automatically discard invalid primitive objects.
	UnregisterCompositeImpl();
}

void UHoldoutCompositeComponent::OnRegister()
{
	Super::OnRegister();

	if (!UE::CompositeCore::IsRegisterPrimitivesOnTickEnabled())
	{
		RegisterCompositeImpl();
	}
}

void UHoldoutCompositeComponent::OnUnregister()
{
	if (!UE::CompositeCore::IsRegisterPrimitivesOnTickEnabled())
	{
		UnregisterCompositeImpl();
	}

	Super::OnUnregister();
}

void UHoldoutCompositeComponent::TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	if (UE::CompositeCore::IsRegisterPrimitivesOnTickEnabled())
	{
		RegisterCompositeImpl();
	}
}

void UHoldoutCompositeComponent::DetachFromComponent(const FDetachmentTransformRules& DetachmentRules)
{
	// Note: We also unregister here while the attached parent pointer is still valid.
	UnregisterCompositeImpl();

	Super::DetachFromComponent(DetachmentRules);
}

#if WITH_EDITOR
void UHoldoutCompositeComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(ThisClass, bIsEnabled))
	{
		if (bIsEnabled)
		{
			RegisterCompositeImpl();
		}
		else
		{
			UnregisterCompositeImpl();
		}
	}
}
#endif

void UHoldoutCompositeComponent::OnComponentCreated()
{
	Super::OnComponentCreated();

	OnComponentCreatedDelegate.Broadcast(this);
}

void UHoldoutCompositeComponent::OnAttachmentChanged()
{
	Super::OnAttachmentChanged();

	UnregisterCompositeImpl();

	USceneComponent* SceneComponent = GetAttachParent();
	if (IsValid(SceneComponent))
	{
		TArray<UPrimitiveComponent*> ParentPrimitives = UE::CompositeCore::Private::FindPrimitiveComponents(SceneComponent);
		if (ParentPrimitives.IsEmpty())
		{
#if WITH_EDITOR
			const UCompositeCorePluginSettings* PluginSettings = GetDefault<UCompositeCorePluginSettings>();

			if (!PluginSettings->IsAllowedComponentClass(SceneComponent))
			{
				/** Utility functions for notifications */
				struct FSuppressDialogOptions
				{
					static bool ShouldSuppressModal()
					{
						bool bSuppressNotification = false;
						GConfig->GetBool(TEXT("CompositeCore"), TEXT("SuppressCompositeCorePrimitiveWarning"), bSuppressNotification, GEditorPerProjectIni);
						return bSuppressNotification;
					}

					static ECheckBoxState GetDontAskAgainCheckBoxState()
					{
						return ShouldSuppressModal() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					}

					static void OnDontAskAgainCheckBoxStateChanged(ECheckBoxState NewState)
					{
						// If the user selects to not show this again, set that in the config so we know about it in between sessions
						const bool bSuppressNotification = (NewState == ECheckBoxState::Checked);
						GConfig->SetBool(TEXT("CompositeCore"), TEXT("SuppressCompositeCorePrimitiveWarning"), bSuppressNotification, GEditorPerProjectIni);
					}
				};

				// Skip if the user has specified to suppress this pop up.
				if (!FSuppressDialogOptions::ShouldSuppressModal())
				{
					FNotificationInfo Info(LOCTEXT("CompositeParentNotification",
						"The composite component must be parented to a primitive component (or one that has primitives)."));
					Info.ExpireDuration = 5.0f;

					// Add a "Don't show this again" option
					Info.CheckBoxState = TAttribute<ECheckBoxState>::Create(&FSuppressDialogOptions::GetDontAskAgainCheckBoxState);
					Info.CheckBoxStateChanged = FOnCheckStateChanged::CreateStatic(&FSuppressDialogOptions::OnDontAskAgainCheckBoxStateChanged);
					Info.CheckBoxText = LOCTEXT("DontShowThisAgainCheckBoxMessage", "Don't show this again");

					FSlateNotificationManager::Get().AddNotification(Info);
				}
			}
#endif
		}
		else
		{
			RegisterCompositeImpl();
		}
	}
}

bool UHoldoutCompositeComponent::IsEnabled() const
{
	return bIsEnabled;
}

void UHoldoutCompositeComponent::SetEnabled(bool bInIsEnabled)
{
	if (bIsEnabled != bInIsEnabled)
	{
		bIsEnabled = bInIsEnabled;

		if (bIsEnabled)
		{
			RegisterCompositeImpl();
		}
		else
		{
			UnregisterCompositeImpl();
		}
	}
}

void UHoldoutCompositeComponent::RegisterCompositeImpl()
{
	using namespace UE::CompositeCore;

	if (!bIsEnabled)
	{
		return;
	}

	UCompositeCoreSubsystem* Subsystem = UWorld::GetSubsystem<UCompositeCoreSubsystem>(GetWorld());
	TArray<UPrimitiveComponent*> ParentPrimitives = Private::FindPrimitiveComponents(GetAttachParent());

	if (IsValid(Subsystem) && !ParentPrimitives.IsEmpty())
	{
		Subsystem->RegisterPrimitives(ParentPrimitives);
	}
}

void UHoldoutCompositeComponent::UnregisterCompositeImpl()
{
	using namespace UE::CompositeCore::Private;

	UCompositeCoreSubsystem* Subsystem = UWorld::GetSubsystem<UCompositeCoreSubsystem>(GetWorld());
	TArray<UPrimitiveComponent*> ParentPrimitives = FindPrimitiveComponents(GetAttachParent());

	if (IsValid(Subsystem) && !ParentPrimitives.IsEmpty())
	{
		Subsystem->UnregisterPrimitives(ParentPrimitives);
	}
}

#undef LOCTEXT_NAMESPACE
