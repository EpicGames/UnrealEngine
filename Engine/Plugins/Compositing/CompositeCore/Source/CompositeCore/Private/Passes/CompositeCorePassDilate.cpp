// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositeCorePassDilate.h"

#include "RHIFeatureLevel.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "PixelShaderUtils.h"
#include "ScreenPass.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"

DECLARE_GPU_STAT_NAMED(FCompositeCoreDilate, TEXT("CompositeCore.Dilate"));

class FCompositeCoreDilateShader : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCompositeCoreDilateShader);
	SHADER_USE_PARAMETER_STRUCT(FCompositeCoreDilateShader, FGlobalShader);

	class FDilationSize : SHADER_PERMUTATION_INT("DILATION_SIZE", 3);
	using FPermutationDomain = TShaderPermutationDomain<FDilationSize>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FScreenPassTextureInput, Input)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Output)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RWOutputTexture)
		SHADER_PARAMETER(uint32, bInvertedAlpha)
		SHADER_PARAMETER(uint32, bOpacifyOutput)
		SHADER_PARAMETER(float, AlphaThreshold)
	END_SHADER_PARAMETER_STRUCT()

	static inline void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("TILE_SIZE"), ThreadGroupSize);

		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}

	static const uint32 ThreadGroupSize = 16;
};
IMPLEMENT_GLOBAL_SHADER(FCompositeCoreDilateShader, "/Plugin/CompositeCore/Private/CompositeCoreDilate.usf", "MainCS", SF_Compute);

namespace UE
{
	namespace CompositeCore
	{
		namespace Private
		{
			void AddDilatePass(FRDGBuilder& GraphBuilder, FRDGTextureRef Input, FRDGTextureRef Output, ERHIFeatureLevel::Type FeatureLevel, const FDilateInputs& PassInputs)
			{
				RDG_EVENT_SCOPE_STAT(GraphBuilder, FCompositeCoreDilate, "CompositeCore.Dilate");

				checkf(Input->Desc.Extent == Output->Desc.Extent,
					TEXT("AddDilatePass: Input (%dx%d) and Output (%dx%d) must have matching extents"),
					Input->Desc.Extent.X, Input->Desc.Extent.Y, Output->Desc.Extent.X, Output->Desc.Extent.Y);
				checkf(EnumHasAnyFlags(Output->Desc.Flags, TexCreate_UAV),
					TEXT("AddDilatePass: Output texture '%s' must have TexCreate_UAV set."), Output->Name);

				FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(FeatureLevel);
				const FIntPoint TextureSize = Input->Desc.Extent;

				// CompositeCore's dilation texture always fills the entire extent with no viewport offset.
				FScreenPassTexture InputScreenPass(Input, FIntRect(FIntPoint::ZeroValue, TextureSize));

				FCompositeCoreDilateShader::FParameters* PassParameters = GraphBuilder.AllocParameters<FCompositeCoreDilateShader::FParameters>();
				PassParameters->Input = GetScreenPassTextureInput(InputScreenPass, TStaticSamplerState<SF_Point>::GetRHI());
				FScreenPassTexture OutputScreenPass(Output, FIntRect(FIntPoint::ZeroValue, TextureSize));
				PassParameters->Output = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(OutputScreenPass));
				PassParameters->RWOutputTexture = GraphBuilder.CreateUAV(Output);
				PassParameters->bInvertedAlpha = 1; // CompositeCore always uses inverted alpha (A=0 opaque, A=1 transparent)
				PassParameters->bOpacifyOutput = PassInputs.bOpacifyOutput;
				PassParameters->AlphaThreshold = PassInputs.AlphaThreshold;

				FCompositeCoreDilateShader::FPermutationDomain PermutationVector;
				PermutationVector.Set<FCompositeCoreDilateShader::FDilationSize>(FMath::Clamp(PassInputs.DilationSize, 0, 2));

				TShaderMapRef<FCompositeCoreDilateShader> ComputeShader(GlobalShaderMap, PermutationVector);
				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("CompositeCore.Dilate (%dx%d)", TextureSize.X, TextureSize.Y),
					GSupportsEfficientAsyncCompute ? ERDGPassFlags::AsyncCompute : ERDGPassFlags::Compute,
					ComputeShader,
					PassParameters,
					FComputeShaderUtils::GetGroupCount(TextureSize, FCompositeCoreDilateShader::ThreadGroupSize)
				);
			}

			FRDGTextureDesc GetPostProcessingDesc(FRDGTextureDesc InDesc, EPixelFormat InOutputFormat)
			{
				FRDGTextureDesc OutputDesc = InDesc;
				OutputDesc.Dimension = ETextureDimension::Texture2D;
				OutputDesc.ArraySize = 1;

				OutputDesc.Reset();
				if (InOutputFormat != PF_Unknown)
				{
					OutputDesc.Format = InOutputFormat;
				}
				OutputDesc.ClearValue = FClearValueBinding(FLinearColor::Black);
				OutputDesc.Flags &= (~ETextureCreateFlags::FastVRAM);

				return OutputDesc;
			}
		}
	}
}
