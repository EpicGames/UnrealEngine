// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RenderGraphFwd.h"
#include "RHIFwd.h"

namespace UE
{
	namespace CompositeCore
	{
		namespace Private
		{
			struct FDilateInputs
			{
				/** Dilation kernel radius in pixels (0 = no-op, 1 = 3×3 neighborhood, 2 = 5×5 neighborhood). Clamped to [0, 2]. */
				int32 DilationSize = 1;

				/** Opacify the pass output to solid colors. */
				bool bOpacifyOutput = true;

				/** Opacity threshold - pixels at or below this (standard opacity space) are treated as fully transparent. */
				float AlphaThreshold = 0.0000001f;
			};

			/**
			* Compute shader dilation pass of non-translucent color texels. This is done as preparation
			* for compositing to hide aliasing under the main render's anti-aliased edges, with an
			* optional opacification step.
			*/
			void AddDilatePass(FRDGBuilder& GraphBuilder, FRDGTextureRef Input, FRDGTextureRef Output, ERHIFeatureLevel::Type FeatureLevel, const FDilateInputs& PassInputs = {});
		}
	}
}
