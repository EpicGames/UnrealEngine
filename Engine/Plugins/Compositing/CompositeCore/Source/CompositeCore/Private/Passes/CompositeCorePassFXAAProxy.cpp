// Copyright Epic Games, Inc. All Rights Reserved.

#include "Passes/CompositeCorePassFXAAProxy.h"

#include "PixelShaderUtils.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "SceneView.h"
#include "ScreenPass.h"
#include "PostProcess/PostProcessAA.h"
#include "PostProcess/PostProcessMaterialInputs.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"

DECLARE_GPU_STAT_NAMED(FCompositeCoreFXAA, TEXT("CompositeCore.FXAA"));
DECLARE_GPU_STAT_NAMED(FCompositeCoreDisplayTransform, TEXT("CompositeCore.DisplayTransform"));

class FCompositeCoreDisplayTransformShader : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCompositeCoreDisplayTransformShader);
	SHADER_USE_PARAMETER_STRUCT(FCompositeCoreDisplayTransformShader, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FScreenPassTextureInput, Input)
		SHADER_PARAMETER(FScreenTransform, SvPositionToInputTextureUV)
		SHADER_PARAMETER(uint32, bIsForward)
		SHADER_PARAMETER(float, Gamma)
		SHADER_PARAMETER(float, InvGamma)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FCompositeCoreDisplayTransformShader, "/Plugin/CompositeCore/Private/CompositeCoreDisplayTransform.usf", "MainPS", SF_Pixel);

namespace UE
{
	namespace CompositeCore
	{
		namespace Private
		{
			FScreenPassTexture AddDisplayTransformPass(FRDGBuilder& GraphBuilder, const FScreenPassViewInfo ViewInfo, const FScreenPassTexture& Input, const FScreenPassRenderTarget& OverrideOutput, bool bIsForward, float InGamma)
			{
				RDG_EVENT_SCOPE_STAT(GraphBuilder, FCompositeCoreDisplayTransform, "CompositeCore.DisplayTransform");

				FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(ViewInfo.FeatureLevel);

				FScreenPassRenderTarget Output = OverrideOutput;

				if (!Output.IsValid())
				{
					FRDGTextureDesc OutputDesc = Input.Texture->Desc;
					OutputDesc.Flags |= TexCreate_RenderTargetable;
					OutputDesc.Format = FCompositeCorePassProxy::GetSceneColorFormatChecked();
					OutputDesc.Extent = Input.ViewRect.Size();

					const FIntRect ZeroBasedViewRect(FIntPoint::ZeroValue, Input.ViewRect.Size());
					Output = FScreenPassRenderTarget(GraphBuilder.CreateTexture(OutputDesc, TEXT("FXAACompositePass")), ZeroBasedViewRect, ERenderTargetLoadAction::ENoAction);
				}

				FCompositeCoreDisplayTransformShader::FParameters* Parameters = GraphBuilder.AllocParameters<FCompositeCoreDisplayTransformShader::FParameters>();
				Parameters->Input = GetScreenPassTextureInput(Input, TStaticSamplerState<>::GetRHI());
				Parameters->SvPositionToInputTextureUV = (
					FScreenTransform::ChangeTextureBasisFromTo(FScreenPassTextureViewport(Output), FScreenTransform::ETextureBasis::TexelPosition, FScreenTransform::ETextureBasis::ViewportUV) *
					FScreenTransform::ChangeTextureBasisFromTo(FScreenPassTextureViewport(Input), FScreenTransform::ETextureBasis::ViewportUV, FScreenTransform::ETextureBasis::TextureUV));
				Parameters->bIsForward = bIsForward;
				Parameters->Gamma = InGamma;
				Parameters->InvGamma = FMath::SafeDivide(1.0f, InGamma);
				Parameters->RenderTargets[0] = Output.GetRenderTargetBinding();

				TShaderMapRef<FCompositeCoreDisplayTransformShader> PixelShader(GlobalShaderMap);
				FPixelShaderUtils::AddFullscreenPass(
					GraphBuilder,
					GlobalShaderMap,
					RDG_EVENT_NAME("Composite.DisplayTransform (%dx%d)", Output.ViewRect.Width(), Output.ViewRect.Height()),
					PixelShader,
					Parameters,
					Output.ViewRect
				);

				return Output;
			}
		}

		FPassTexture FFXAAPassProxy::Add(FRDGBuilder& GraphBuilder, const FSceneView& InView, const FPassInputArray& Inputs, const FPassContext& PassContext) const
		{
			RDG_EVENT_SCOPE_STAT(GraphBuilder, FCompositeCoreFXAA, "CompositeCore.FXAA");

			check(ValidateInputs(Inputs));

			FScreenPassTexture Input = Inputs[0].Texture;
			FResourceMetadata Metadata = Inputs[0].Metadata;
			const bool bLinearSourceColors = (Metadata.Encoding == EEncoding::Linear);
			const bool bApplyDisplayTransform = bLinearSourceColors && bDisplayTransform;

			if (bApplyDisplayTransform)
			{
				// We tonemap & encode the result so that FXAA can operate on perceptual colors
				constexpr bool bIsForward = true;
				Input = Private::AddDisplayTransformPass(GraphBuilder, InView, Input, {}, bIsForward);
			}

			static IConsoleVariable* FXAAQualityCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.FXAA.Quality"));
			check(FXAAQualityCVar);
			const int32 Quality = QualityOverride.IsSet() ? QualityOverride.GetValue() : FXAAQualityCVar->GetInt();

			FFXAAInputs FXAAInputs{};
			FXAAInputs.SceneColor = Input;
			FXAAInputs.Quality = static_cast<EFXAAQuality>(FMath::Clamp(Quality, 0, static_cast<int32>(EFXAAQuality::MAX) - 1));
			if (!bApplyDisplayTransform)
			{
				FXAAInputs.OverrideOutput = Inputs.OverrideOutput.IsValid()
					? Inputs.OverrideOutput
					: CreateOutputRenderTarget(GraphBuilder, InView, PassContext.OutputViewRect, Input.Texture->Desc, TEXT("CompositeCorePassFXAA"));
			}

			FScreenPassTexture Output = AddFXAAPass(GraphBuilder, InView, FXAAInputs);

			if (bApplyDisplayTransform)
			{
				// We decode and invert the tonemapping to obtain linear colors again.
				constexpr bool bIsForward = false;
				Output = Private::AddDisplayTransformPass(GraphBuilder, InView, Output, Inputs.OverrideOutput, bIsForward);
			}

			return FPassTexture{ MoveTemp(Output), Metadata };
		}
	}
}
