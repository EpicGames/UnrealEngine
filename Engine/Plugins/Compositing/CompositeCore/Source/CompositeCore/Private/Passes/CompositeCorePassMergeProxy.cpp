// Copyright Epic Games, Inc. All Rights Reserved.

#include "Passes/CompositeCorePassMergeProxy.h"

#include "CompositeCoreSettings.h"
#include "CompositeCoreModule.h"

#include "Logging/StructuredLog.h"
#include "PixelShaderUtils.h"
#include "PostProcess/LensDistortion.h"
#include "PostProcess/PostProcessMaterialInputs.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "SceneManagement.h"
#include "SceneView.h"
#include "SystemTextures.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CompositeCorePassMergeProxy)

DECLARE_GPU_STAT_NAMED(FCompositeCoreMerge, TEXT("CompositeCore.Merge"));

static TAutoConsoleVariable<int32> CVarCompositeCoreApplyPreExposure(
	TEXT("CompositeCore.ApplyPreExposure"),
	0,
	TEXT("When enabled, the scene main render pre-exposure is applied onto separate composited render(s)."),
	ECVF_RenderThreadSafe);

class FCompositeCoreMergeShader : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCompositeCoreMergeShader);
	SHADER_USE_PARAMETER_STRUCT(FCompositeCoreMergeShader, FGlobalShader);

	class FUseDistortion : SHADER_PERMUTATION_BOOL("USE_DISTORTION");
	using FPermutationDomain = TShaderPermutationDomain<FUseDistortion>;

	BEGIN_SHADER_PARAMETER_STRUCT(FInputPassTextureParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, Texture)
		SHADER_PARAMETER_SAMPLER(SamplerState, Sampler)
		SHADER_PARAMETER(uint32, bInvertedAlpha)
		SHADER_PARAMETER(uint32, SourceEncoding)
		SHADER_PARAMETER(uint32, DistortionUV)
		SHADER_PARAMETER(float, Exposure)
	END_SHADER_PARAMETER_STRUCT()

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Input0)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Input1)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Output)
		SHADER_PARAMETER_STRUCT(FInputPassTextureParameters, Tex0)
		SHADER_PARAMETER_STRUCT(FInputPassTextureParameters, Tex1)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DistortingDisplacementTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, DistortingDisplacementSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, UndistortingDisplacementTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, UndistortingDisplacementSampler)
		SHADER_PARAMETER(FVector2f, DisplayGamma)
		SHADER_PARAMETER(uint32, MergeOp)
		SHADER_PARAMETER(uint32, OutputEncoding)
		SHADER_PARAMETER(uint32, bFlags)
		SHADER_PARAMETER(uint32, bBlendInDisplaySpace)
		SHADER_PARAMETER(uint32, bLuminanceOnly)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FCompositeCoreMergeShader, "/Plugin/CompositeCore/Private/CompositeCoreMerge.usf", "MainPS", SF_Pixel);

namespace UE
{
	namespace CompositeCore
	{
		enum class EDistortionUV
		{
			None = 0,
			Distorted = 1,
			Undistorted = 2
		};

		FCompositeCoreMergeShader::FInputPassTextureParameters GetPassTextureParameters(
			const UE::CompositeCore::FPassInput& Input,
			const FResourceMetadata& OutputMetadata,
			const FPassContext& PassContext,
			const FSceneViewFamily* Family,
			const bool bIsLensDistortionActive,
			const float InPreExposure)
		{
			FCompositeCoreMergeShader::FInputPassTextureParameters Parameters;

			Parameters.Texture = Input.Texture.Texture;
			Parameters.Sampler = Input.Metadata.GetSamplerState();
			Parameters.bInvertedAlpha = Input.Metadata.bInvertedAlpha;
			Parameters.SourceEncoding = static_cast<uint32>(Input.Metadata.Encoding);
			Parameters.Exposure = 1.0f;

			if (CVarCompositeCoreApplyPreExposure.GetValueOnRenderThread())
			{
				// When we output back to scene color, optionally adjust input exposure so that the ouput composite matches.
				if (PassContext.bOutputSceneColor && !Input.Metadata.bPreExposed)
				{
					Parameters.Exposure = InPreExposure;
				}
			}

			Parameters.DistortionUV = static_cast<uint32>(EDistortionUV::None);

			if (bIsLensDistortionActive)
			{
				const bool bDistortedInput = Input.Metadata.bDistorted;
				const bool bDistortedOutput = OutputMetadata.bDistorted;

				// Distortion mismatch?
				if (bDistortedInput != bDistortedOutput)
				{
					// Sample texture according to its input distortion
					Parameters.DistortionUV = static_cast<uint32>(bDistortedInput ? EDistortionUV::Distorted : EDistortionUV::Undistorted);
				}
			}

			return Parameters;
		}

		FMergePassProxy::FMergePassProxy(FPassInputDeclArray InPassDeclaredInputs, ECompositeCoreMergeOp InMergeOp, const TCHAR* InParentName, ELensDistortionHandling bInUseLensDistortion, bool bInUltimatteBlend)
			: FCompositeCorePassProxy(MoveTemp(InPassDeclaredInputs))
			, MergeOp(InMergeOp)
			, ParentName(InParentName)
			, bUseLensDistortion(bInUseLensDistortion)
			, bUltimatteBlend(bInUltimatteBlend)
		{
		}

		FPassTexture FMergePassProxy::Add(FRDGBuilder& GraphBuilder, const FSceneView& InView, const FPassInputArray& Inputs, const FPassContext& PassContext) const
		{
			check(ValidateInputs(Inputs));
			RDG_EVENT_SCOPE_STAT(GraphBuilder, FCompositeCoreMerge, "CompositeCore.Merge (%s) %dx%d",
				ParentName,
				Inputs[0].Texture.ViewRect.Width(),
				Inputs[0].Texture.ViewRect.Height()
			);

			UE_LOGF(LogCompositeCore, VeryVerbose, "CompositeCore.Merge: %ls", ParentName);

			FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(InView.GetFeatureLevel());
			const FSceneViewFamily* Family = InView.Family;
			FResourceMetadata OutMetadata = {};

			const FLensDistortionLUT& LensDistortionLUT = LensDistortion::GetLUTUnsafe(InView);
			const bool bIsLensDistortionActive = static_cast<bool>(bUseLensDistortion) && LensDistortionLUT.IsEnabled()
				&& PassContext.Location >= ISceneViewExtension::EPostProcessingPass::SSRInput;

			if (bIsLensDistortionActive && PassContext.bOutputSceneColor)
			{
				const LensDistortion::EPassLocation EngineLocation = LensDistortion::GetPassLocationUnsafe(InView);

				// Optionally set the output as distorted on passes after SSR input when TSR distorts.
				OutMetadata.bDistorted = (PassContext.Location > ISceneViewExtension::EPostProcessingPass::SSRInput) &&
					(EngineLocation == LensDistortion::EPassLocation::TSR);
			}
			else
			{
				// Distortion passthrough
				OutMetadata.bDistorted = Inputs[0].Metadata.bDistorted;
			}


			if (PassContext.bOutputSceneColor)
			{
				OutMetadata.bInvertedAlpha = true;

				if (PassContext.Location >= ISceneViewExtension::EPostProcessingPass::Tonemap)
				{
					if ((Family->EngineShowFlags.Tonemapper == 0) || (Family->EngineShowFlags.PostProcessing == 0))
					{
						OutMetadata.Encoding = EEncoding::Gamma;
					}
					else if (Family->SceneCaptureSource == SCS_FinalColorLDR)
					{
						OutMetadata.Encoding = EEncoding::sRGB;
					}
				}
			}

			const float PreExposure = InView.State ? InView.State->GetPreExposure() : 1.0f;

			FScreenPassRenderTarget Output = Inputs.OverrideOutput;
			if (!Output.IsValid())
			{
				const FRDGTextureDesc Desc = Inputs[0].Texture.Texture->Desc;

				FString& DynamicDebugName = *GraphBuilder.AllocObject<FString>();
				DynamicDebugName = FString::Printf(TEXT("%sOutput"), ParentName);

				Output = CreateOutputRenderTarget(GraphBuilder, InView, PassContext.OutputViewRect, Desc, *DynamicDebugName);
			}

			FCompositeCoreMergeShader::FParameters* PassParameters = GraphBuilder.AllocParameters<FCompositeCoreMergeShader::FParameters>();
			PassParameters->View = InView.ViewUniformBuffer;
			PassParameters->Input0 = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(Inputs[0].Texture));
			PassParameters->Input1 = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(Inputs[1].Texture));
			PassParameters->Output = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(Output));
			PassParameters->Tex0 = GetPassTextureParameters(Inputs[0], OutMetadata, PassContext, Family, bIsLensDistortionActive, PreExposure);
			PassParameters->Tex1 = GetPassTextureParameters(Inputs[1], OutMetadata, PassContext, Family, bIsLensDistortionActive, PreExposure);
			if (bIsLensDistortionActive)
			{
				PassParameters->DistortingDisplacementTexture = LensDistortionLUT.DistortingDisplacementTexture;
				PassParameters->UndistortingDisplacementTexture = LensDistortionLUT.UndistortingDisplacementTexture;
			}
			else
			{
				PassParameters->UndistortingDisplacementTexture = GSystemTextures.GetBlackDummy(GraphBuilder);
				PassParameters->DistortingDisplacementTexture = GSystemTextures.GetBlackDummy(GraphBuilder);
			}
			PassParameters->UndistortingDisplacementSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
			PassParameters->DistortingDisplacementSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
			PassParameters->DisplayGamma = FVector2f(Family->RenderTarget->GetDisplayGamma(), 1.0f / Family->RenderTarget->GetDisplayGamma());
			PassParameters->MergeOp = static_cast<uint32>(MergeOp);
			PassParameters->OutputEncoding = static_cast<uint32>(OutMetadata.Encoding);
			PassParameters->bFlags = static_cast<uint32>(PassContext.bOutputSceneColor);
			PassParameters->bBlendInDisplaySpace = static_cast<uint32>(bUltimatteBlend);
			PassParameters->bLuminanceOnly = static_cast<uint32>(bLuminanceOnly);
			PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();

			FCompositeCoreMergeShader::FPermutationDomain PermutationVector;
			PermutationVector.Set<FCompositeCoreMergeShader::FUseDistortion>(bIsLensDistortionActive);
			TShaderMapRef<FCompositeCoreMergeShader> PixelShader(GlobalShaderMap, PermutationVector);

			FPixelShaderUtils::AddFullscreenPass(
				GraphBuilder,
				GlobalShaderMap,
				RDG_EVENT_NAME("CompositeCore.Merge (%dx%d) PS",
					Output.ViewRect.Width(), Output.ViewRect.Height()),
				PixelShader,
				PassParameters,
				Output.ViewRect
			);

			return FPassTexture{ MoveTemp(Output), MoveTemp(OutMetadata) };
		}

		const FName& GetMergePassPassName()
		{
			return FMergePassProxy::GetTypeNameStatic();
		}
}
}

