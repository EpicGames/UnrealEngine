// Copyright Epic Games, Inc. All Rights Reserved.

#include "Passes/CompositeCorePassProxy.h"

#include "CompositeCoreModule.h"
#include "Passes/CompositeCorePassMergeProxy.h"

#include "FXRenderingUtils.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RHIDefinitions.h"
#include "SceneView.h"
#include "PostProcess/LensDistortion.h"
#include "PostProcess/PostProcessMaterialInputs.h"

static TAutoConsoleVariable<int32> CVarCompositeCoreDebugDilationSize(
	TEXT("CompositeCore.Debug.DilationSize"),
	1,
	TEXT("Size of the pixel dilation applied onto the composite custom render pass. 0, 1 & 2 are supported."),
	ECVF_Default);

namespace UE
{
	namespace CompositeCore
	{
		FBuiltInRenderPassOptions::FBuiltInRenderPassOptions()
		{
			DilationSize = CVarCompositeCoreDebugDilationSize.GetValueOnGameThread();
		}

		const FRenderWork& FRenderWork::GetDefault()
		{
			static FRenderWork DefaultRenderWork = []() -> FRenderWork
				{
					FRenderWork RenderWork;
					TArray<const FCompositeCorePassProxy*>& SSRInputWork = RenderWork.FramePasses.Add(ISceneViewExtension::EPostProcessingPass::SSRInput);
					TArray<const FCompositeCorePassProxy*>& AfterTonemapWork = RenderWork.FramePasses.Add(ISceneViewExtension::EPostProcessingPass::Tonemap);

					const FPassInputDeclArray PassDeclaredInputs = {
						MakeInternalInput(),
						MakeExternalInput(ResourceId::BuiltInCRP),
					};

					SSRInputWork.Add(RenderWork.FrameAllocator->Create<FMergePassProxy>(PassDeclaredInputs));
					AfterTonemapWork.Add(RenderWork.FrameAllocator->Create<FMergePassProxy>(PassDeclaredInputs));

					return RenderWork;
				}();

			return DefaultRenderWork;
		}

		FRenderWork::FRenderWork()
			: ExternalInputs{}
			, PreprocessingPasses{}
			, FramePasses{}
			, FrameAllocator{new FSceneRenderingBulkObjectAllocator}
			, MainRenderMode{}
		{ }

		bool FRenderWork::IsEmpty() const
		{
			bool bIsEmpty = true;

			for (const TPair<ResourceId, TArray<const FCompositeCorePassProxy*>>& Pair : PreprocessingPasses)
			{
				bIsEmpty &= Pair.Value.IsEmpty();
			}

			for (const TPair<ISceneViewExtension::EPostProcessingPass, TArray<const FCompositeCorePassProxy*>>& Pair : FramePasses)
			{
				bIsEmpty &= Pair.Value.IsEmpty();
			}

			bIsEmpty &= !MainRenderMode.IsSet();
			bIsEmpty &= SceneCapturesUpdateQueue.IsEmpty();

			return bIsEmpty;
		}

		FPassInputDeclArray GetDefaultInputDeclArray()
		{
			return FPassInputDeclArray{ MakeInternalInput() };
		}

		FPassInputDecl MakeInternalInput(int32 Index, bool bOriginalCopyBeforePasses)
		{
			FPassInputDecl Decl;
			Decl.Set<FPassInternalResourceDesc>({ Index, bOriginalCopyBeforePasses });
			return Decl;
		}

		FPassInputDecl MakeExternalInput(ResourceId Id)
		{
			FPassInputDecl Decl;
			Decl.Set<FPassExternalResourceDesc>({ Id });
			return Decl;
		}

		FPassInputDecl MakePassInput(const FCompositeCorePassProxy* Pass)
		{
			FPassInputDecl Decl;
			Decl.Set<const FCompositeCorePassProxy*>(Pass);
			return Decl;
		}
	}
}

FRHISamplerState* UE::CompositeCore::FResourceMetadata::GetSamplerState() const
{
	return (Filter == SF_Point)
		? TStaticSamplerState<SF_Point>::GetRHI()
		: TStaticSamplerState<SF_Bilinear>::GetRHI();
}

FCompositeCorePassProxy::FCompositeCorePassProxy(UE::CompositeCore::FPassInputDeclArray InPassDeclaredInputs)
	: PassDeclaredInputs(MoveTemp(InPassDeclaredInputs))
	, PassDeclaredPrimaryOutputOverride()
{ }

UE::CompositeCore::FPassTypeDescriptor FCompositeCorePassProxy::GetTypeDescriptor() const
{
	using namespace UE::CompositeCore;

	// Default implementation: synthesize a descriptor with fallback slot names. Subclasses that
	// need stable slot names or arity constraints should override this with their own descriptor.
	FPassTypeDescriptor Descriptor;
	Descriptor.TypeName = GetTypeName();
	Descriptor.bSupportsVariadicInputs = true;

	const int32 NumInputs = PassDeclaredInputs.Num();
	Descriptor.InputSlotNames.Reserve(NumInputs);
	for (int32 SlotIndex = 0; SlotIndex < NumInputs; ++SlotIndex)
	{
		Descriptor.InputSlotNames.Add(*FString::Printf(TEXT("Input%d"), SlotIndex));
	}

	return Descriptor;
}

void FCompositeCorePassProxy::SetDeclaredInput(int32 InputIndex, UE::CompositeCore::FPassInputDecl InInput)
{
	checkf(PassDeclaredInputs.IsValidIndex(InputIndex),
		TEXT("SetDeclaredInput index %d is out of range [0, %d). Arity is fixed at construction time."),
		InputIndex, PassDeclaredInputs.Num());

	PassDeclaredInputs[InputIndex] = MoveTemp(InInput);
}

FScreenPassRenderTarget FCompositeCorePassProxy::CreateOutputRenderTarget(FRDGBuilder& GraphBuilder,
	const FSceneView& InView, const FIntRect& OutputViewRect, FRDGTextureDesc OutputDesc, const TCHAR* InName)
{
	OutputDesc.Format	= GetSceneColorFormatChecked();
	OutputDesc.NumMips	= 1;
	OutputDesc.Depth	= 1;
	OutputDesc.Flags	= TexCreate_RenderTargetable | TexCreate_ShaderResource | TexCreate_UAV;
	OutputDesc.Extent	= OutputViewRect.Size();

	const FIntRect ZeroBasedViewRect(FIntPoint::ZeroValue, OutputViewRect.Size());
	return FScreenPassRenderTarget(GraphBuilder.CreateTexture(OutputDesc, InName), ZeroBasedViewRect, InView.GetOverwriteLoadAction());
}

EPixelFormat FCompositeCorePassProxy::GetSceneColorFormatChecked(EPixelFormat Fallback)
{
	const EPixelFormat SceneColorFormat = FSceneTexturesConfig::Get().ColorFormat;
	if (SceneColorFormat != PF_Unknown)
	{
		return SceneColorFormat;
	}

	UE_CALL_ONCE([Fallback]()
	{
		UE_LOGF(LogCompositeCore, Warning,
			"FSceneTexturesConfig color format is PF_Unknown; falling back to %ls. "
			"Compositing should not be active on this view.",
			GetPixelFormatString(Fallback));
	});
	return Fallback;
}

bool FCompositeCorePassProxy::ValidateInputs(const UE::CompositeCore::FPassInputArray& Inputs) const
{
	const UE::CompositeCore::FPassTypeDescriptor Descriptor = GetTypeDescriptor();

	// If the pass declares a fixed input count and is not variadic, enforce the count exactly.
	if (Descriptor.FixedInputCount.IsSet() && !Descriptor.bSupportsVariadicInputs)
	{
		return Inputs.Num() == Descriptor.FixedInputCount.GetValue();
	}

	// Otherwise require at least the declared input count, with a minimum of one.
	return Inputs.Num() >= FMath::Max(1, PassDeclaredInputs.Num());
}

UE::CompositeCore::FPassInputArray::FPassInputArray(
	FRDGBuilder& GraphBuilder,
	const FSceneView& InView,
	const FPostProcessMaterialInputs& InPostInputs,
	const ISceneViewExtension::EPostProcessingPass& InLocation)
{
	using namespace UE::CompositeCore;

	Inputs.Reserve(kPostProcessMaterialInputCountMax);

	{
		FPassInput& Input = Inputs.AddDefaulted_GetRef();
		Input.Texture = FScreenPassTexture::CopyFromSlice(GraphBuilder, InPostInputs.GetInput(EPostProcessMaterialInput::SceneColor));
		Input.Metadata.bInvertedAlpha = true;
		Input.Metadata.bPreExposed = true;
		
		// Note: This assumes the lens file is using the "SVE" method, the PPM one isn't engine-registered.
		const FLensDistortionLUT& LensDistortionLUT = LensDistortion::GetLUTUnsafe(InView);
		Input.Metadata.bDistorted = LensDistortionLUT.IsEnabled()
			&& (LensDistortion::GetPassLocationUnsafe(InView) == LensDistortion::EPassLocation::TSR)
			&& InLocation >= ISceneViewExtension::EPostProcessingPass::SSRInput;

		// After-tonemap scene color may have encoding manually applied, as opposed to _SRGB textures
		if (InLocation >= ISceneViewExtension::EPostProcessingPass::Tonemap)
		{
			if ((InView.Family->EngineShowFlags.Tonemapper == 0) || (InView.Family->EngineShowFlags.PostProcessing == 0))
			{
				Input.Metadata.Encoding = UE::CompositeCore::EEncoding::Gamma;
			}
			else if (InView.Family->SceneCaptureSource == SCS_FinalColorLDR)
			{
				Input.Metadata.Encoding = UE::CompositeCore::EEncoding::sRGB;
			}
		}
	}

	{
		FPassInput& Input = Inputs.AddDefaulted_GetRef();
		Input.Texture = FScreenPassTexture::CopyFromSlice(GraphBuilder, InPostInputs.GetInput(EPostProcessMaterialInput::SeparateTranslucency));
		Input.Metadata.bInvertedAlpha = true;
	}

	{
		FPassInput& Input = Inputs.AddDefaulted_GetRef();
		Input.Texture = FScreenPassTexture::CopyFromSlice(GraphBuilder, InPostInputs.GetInput(EPostProcessMaterialInput::CombinedBloom));
		Input.Metadata.bInvertedAlpha = true;
	}

	{
		FPassInput& Input = Inputs.AddDefaulted_GetRef();
		Input.Texture = FScreenPassTexture::CopyFromSlice(GraphBuilder, InPostInputs.GetInput(EPostProcessMaterialInput::PostTonemapHDRColor));
		Input.Metadata.bInvertedAlpha = true;
	}

	{
		FPassInput& Input = Inputs.AddDefaulted_GetRef();
		Input.Texture = FScreenPassTexture::CopyFromSlice(GraphBuilder, InPostInputs.GetInput(EPostProcessMaterialInput::Velocity));
	}

	OverrideOutput = InPostInputs.OverrideOutput;
}

FPostProcessMaterialInputs UE::CompositeCore::FPassInputArray::ToPostProcessInputs(FRDGBuilder& GraphBuilder, FSceneTextureShaderParameters SceneTextures) const
{
	using namespace UE::CompositeCore;

	FPostProcessMaterialInputs Result;
	for (int32 Index = 0; Index < Inputs.Num(); ++Index)
	{
		const FPassInput& ResolvedInput = Inputs[Index];
		Result.SetInput(static_cast<EPostProcessMaterialInput>(Index), FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, ResolvedInput.Texture));
	}
	Result.SceneTextures = MoveTemp(SceneTextures);

	return Result;
}
