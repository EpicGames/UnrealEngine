// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Logging/LogMacros.h"
#include "Stats/Stats.h"

COMPOSITECORE_API DECLARE_LOG_CATEGORY_EXTERN(LogCompositeCore, Log, All);

DECLARE_STATS_GROUP(TEXT("CompositeCore"), STATGROUP_CompositeCore, STATCAT_Advanced);

class FCompositeCoreModule : public IModuleInterface
{
public:
	// IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	// End of IModuleInterface interface
};

