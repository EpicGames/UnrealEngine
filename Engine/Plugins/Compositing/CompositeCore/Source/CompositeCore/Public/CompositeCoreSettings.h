// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "Engine/DeveloperSettings.h"

#include "CompositeCoreSettings.generated.h"

class UPrimitiveComponent;
class USceneComponent;

namespace UE
{
	namespace CompositeCore
	{
		COMPOSITECORE_API bool IsRegisterPrimitivesOnTickEnabled();
	}
}

/**
 * Settings for the CompositeCore module.
 */
UCLASS(MinimalAPI, config = Engine, defaultconfig, meta = (DisplayName = "Composite Core"))
class UCompositeCorePluginSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UCompositeCorePluginSettings();

	//~ Begin UDeveloperSettings interface
	virtual FName GetCategoryName() const;
#if WITH_EDITOR
	virtual FText GetSectionText() const override;
	virtual FName GetSectionName() const override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UDeveloperSettings interface

	//~ Begin UObject Interface
	virtual void PostInitProperties() override;
	//~ End UObject Interface

	// Ensure that the primitive component class is allowed by checking against plugin settings.
	COMPOSITECORE_API bool IsAllowedPrimitiveClass(const UPrimitiveComponent* InPrimitiveComponent) const;

	// Ensure that the scene component class is allowed by checking against plugin settings.
	COMPOSITECORE_API bool IsAllowedComponentClass(const USceneComponent* InComponent) const;

public:
	UPROPERTY(config, EditAnywhere, Category = CompositeCore, meta = (
		ConsoleVariable = "CompositeCore.ApplyPreExposure", DisplayName = "Apply Pre-Exposure",
		ToolTip = "When enabled, the scene main render pre-exposure is applied onto the separate composited render. This can be used to match exposure to the scene. (Maps to \"CompositeCore.ApplyPreExposure\" console variable).",
		ConfigRestartRequired = false))
	uint32 bApplyPreExposure : 1;

	/** Primitive component classes that do not support the composite pipeline.*/
	UPROPERTY(config, EditAnywhere, Category = CompositeCore)
	TArray<FSoftClassPath> DisabledPrimitiveClasses;

	/** Allowed component classes for which users will not be warned if associated primitive cannot immediately be found. */
	UPROPERTY(config, EditAnywhere, Category = CompositeCore)
	TArray<FSoftClassPath> AllowedComponentClasses;

	/** Composite (scene view extension) post-processing priority, which defaults to before OpenColorIO. */
	UPROPERTY(config, EditAnywhere, Category = CompositeCore, AdvancedDisplay)
	int32 SceneViewExtensionPriority;

private:

	/** Cache primitive class pointers to prevent load object calls during allowed class checks. */
	void CacheDisabledPrimitiveClasses() const;

	/** Cache component class pointers to prevent load object calls during allowed class checks. */
	void CacheAllowedComponentClasses() const;

	/** Cached list of disallowed primitive class types. */
	mutable TArray<const UClass*> CachedDisabledPrimitiveClasses;

	/** Dirty primitive cache flag. */
	mutable bool bIsPrimitiveCacheDirty;

	/** Cached list of allowed component class types. */
	mutable TArray<const UClass*> CachedAllowedComponentClasses;

	/** Dirty component cache flag. */
	mutable bool bIsComponentCacheDirty;

	/** Mutable cache lock. */
	mutable FCriticalSection CriticalSection;
};


