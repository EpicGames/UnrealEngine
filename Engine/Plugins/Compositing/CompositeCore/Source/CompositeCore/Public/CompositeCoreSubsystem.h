// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MaterialShared.h"
#include "Subsystems/WorldSubsystem.h"

#include "CompositeCoreTypes.h"
#include "CompositeCoreSubsystem.generated.h"

#define UE_API COMPOSITECORE_API

class FCompositeCoreSceneViewExtension;
struct FCompositeCoreSettings;
class SNotificationItem;
class UPrimitiveComponent;

namespace UE::CompositeCore
{
	struct FBuiltInRenderPassOptions;
	struct FRenderWork;
}

/**
 * Composite subsytem used as an interface to the (private) scene view extension.
 */
UCLASS(MinimalAPI, BlueprintType, Transient)
class UCompositeCoreSubsystem : public UTickableWorldSubsystem
{
	GENERATED_BODY()

public:
	UE_API UCompositeCoreSubsystem();

	// USubsystem implementation Begin
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;
	// USubsystem implementation End

	// FTickableGameObject implementation Begin
	virtual bool IsTickableInEditor() const override { return true; }
	UE_API virtual void Tick(float DeltaTime) override;
	// FTickableGameObject implementation End

	virtual TStatId GetStatId() const override { RETURN_QUICK_DECLARE_CYCLE_STAT(UCompositeCoreSubsystem, STATGROUP_Tickables); }

	/* Register a single primitive for compositing. */
	UFUNCTION(BlueprintCallable, Category = "Composite Core")
	UE_API void RegisterPrimitive(UPrimitiveComponent* InPrimitiveComponent, ECompositeCoreHoldoutManagement HoldoutManagement = ECompositeCoreHoldoutManagement::Automatic);

	/* Register multiple primitives for compositing. */
	UE_API void RegisterPrimitives(const TArray<UPrimitiveComponent*>& InPrimitiveComponents, ECompositeCoreHoldoutManagement HoldoutManagement = ECompositeCoreHoldoutManagement::Automatic);

	/* Unregister a single primitive from compositing. */
	UFUNCTION(BlueprintCallable, Category = "Composite Core")
	UE_API void UnregisterPrimitive(UPrimitiveComponent* InPrimitiveComponent, ECompositeCoreHoldoutManagement HoldoutManagement = ECompositeCoreHoldoutManagement::Automatic);

	/* Unregister multiple primitives from compositing. */
	UE_API void UnregisterPrimitives(const TArray<UPrimitiveComponent*>& InPrimitiveComponents, ECompositeCoreHoldoutManagement HoldoutManagement = ECompositeCoreHoldoutManagement::Automatic);

	/* Set scene view extension frame render work. */
	UE_API void SetRenderWork(UE::CompositeCore::FRenderWork&& InWork);

	/* Reset scene view extension frame render work. */
	UE_API void ResetRenderWork();

	/* Set built-in composite render pass options. */
	UE_API void SetBuiltInRenderPassOptions(const UE::CompositeCore::FBuiltInRenderPassOptions& InOptions);

	/* Reset built-in composite render pass options. */
	UE_API void ResetBuiltInRenderPassOptions();

	/** True if the project settings are valid for the CompositeCore plugin to work. */
	static UE_API bool IsProjectSettingsValid();

private:

	/* Returns true if the (renderer) project settings are correctly enabled for the composite to be active. */
	UE_API bool ValidateProjectSettings();

#if WITH_EDITOR
	/* Toast notification to ask users to enable the missing project settings. */
	UE_API void PrimitiveHoldoutSettingsNotification();

	/* Toast notification item. */
	TWeakPtr<SNotificationItem> HoldoutNotificationItem;
#endif

	/* Owned scene view extension. */
	TSharedPtr<FCompositeCoreSceneViewExtension, ESPMode::ThreadSafe> CompositeCoreViewExtension;
};

#undef UE_API
