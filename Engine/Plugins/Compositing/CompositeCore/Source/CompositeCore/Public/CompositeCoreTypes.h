// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/ObjectMacros.h"

#include "CompositeCoreTypes.generated.h"

#define UE_API COMPOSITECORE_API

/* Holdout property management on registered primitives. */
UENUM(BlueprintType)
enum class ECompositeCoreHoldoutManagement : uint8
{
	None,
	Automatic,
};

#undef UE_API
