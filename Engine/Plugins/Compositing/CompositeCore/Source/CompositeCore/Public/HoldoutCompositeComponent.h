// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Components/SceneComponent.h"

#include "HoldoutCompositeComponent.generated.h"

class UHoldoutCompositeComponent;

DECLARE_MULTICAST_DELEGATE_OneParam(FHoldoutCompositeComponentDelegate, const UHoldoutCompositeComponent*);

UCLASS(MinimalAPI, ClassGroup = Rendering, HideCategories=(Activation, Transform, Lighting, Rendering, Tags, Cooking, Physics, LOD, AssetUserData, Navigation), editinlinenew, meta = (BlueprintSpawnableComponent))
class UHoldoutCompositeComponent : public USceneComponent
{
	GENERATED_UCLASS_BODY()

public:
	//~ Begin UActorComponent interface
	virtual void BeginDestroy() override;
	//~ End UActorComponent interface

	//~ Begin UActorComponent interface
	virtual void OnRegister() override;
	virtual void OnUnregister() override;
	virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	//~ End UActorComponent interface
	
	//~ Begin USceneComponent Interface.
	virtual void OnComponentCreated() override;
	virtual void OnAttachmentChanged() override;
	virtual void DetachFromComponent(const FDetachmentTransformRules& DetachmentRules) override;
	//~ End USceneComponent Interface

#if WITH_EDITOR
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif //WITH_EDITOR

	/* Get the enabled state of the component. */
	UFUNCTION(BlueprintGetter)
	COMPOSITECORE_API bool IsEnabled() const;

	/* Set the enabled state of the component. */
	UFUNCTION(BlueprintSetter)
	COMPOSITECORE_API void SetEnabled(bool bInEnabled);

	/** Event fired once a component is created. */
	COMPOSITECORE_API static FHoldoutCompositeComponentDelegate OnComponentCreatedDelegate;

private:

	/* Private implementation of the register method. */
	void RegisterCompositeImpl();

	/* Private implementation of the unregister method. */
	void UnregisterCompositeImpl();

private:

	/* Whether or not the component activates the composite. */
	UPROPERTY(EditAnywhere, BlueprintGetter = IsEnabled, BlueprintSetter = SetEnabled, Category = "CompositeCore", meta = (AllowPrivateAccess = true))
	bool bIsEnabled = true;
};

