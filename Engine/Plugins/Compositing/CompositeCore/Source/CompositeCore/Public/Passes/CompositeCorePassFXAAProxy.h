// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "CompositeCorePassProxy.h"

#define UE_API COMPOSITECORE_API

namespace UE
{
	namespace CompositeCore
	{
		namespace Private
		{
			UE_API FScreenPassTexture AddDisplayTransformPass(FRDGBuilder& GraphBuilder, const FScreenPassViewInfo ViewInfo, const FScreenPassTexture& Input, const FScreenPassRenderTarget& OverrideOutput, bool bIsForward, float InGamma = 2.2f);
		}

		class FFXAAPassProxy : public FCompositeCorePassProxy
		{
		public:
			IMPLEMENT_COMPOSITE_PASS(FFXAAPassProxy);

			using FCompositeCorePassProxy::FCompositeCorePassProxy;

			UE_API FPassTexture Add(FRDGBuilder& GraphBuilder, const FSceneView& InView, const FPassInputArray& Inputs, const FPassContext& PassContext) const override;

			/** Optional r.FXAA.Quality setting override.  */
			TOptional<int32> QualityOverride = {};

			/** When true, applies a display transform (tonemap + gamma) before AA and inverts it after. Improves edge detection on linear content but may produce dark halos at alpha boundaries. */
			bool bDisplayTransform = true;
		};
	}
}

#undef UE_API
