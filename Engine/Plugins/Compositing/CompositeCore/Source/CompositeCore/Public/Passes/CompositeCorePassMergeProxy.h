// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "CompositeCorePassProxy.h"

#include "CompositeCorePassMergeProxy.generated.h"

#define UE_API COMPOSITECORE_API

/**
* Merge/blend operations, assuming inputs are alpha pre-multiplied.
*/
UENUM()
enum class ECompositeCoreMergeOp : uint8
{
	None UMETA(ToolTip = "Current layer A previous replaces previous layer B."),
	Over UMETA(ToolTip = "Current layer A over previous layer B: A + B * (1-a)."),
	Under UMETA(ToolTip = "Current layer A under previous layer B: A * (1-b) + B."),
	Add UMETA(ToolTip = "Current layer A added to previous layer B: A + B."),
	Subtract UMETA(ToolTip = "Previous layer B subtracted from current layer A: A - B."),
	Multiply UMETA(ToolTip = "Current layer A multiplied by previous layer B: A * B."),
	Divide UMETA(ToolTip = "Current layer A (safely) divided by previous layer B: A / B."),
	Min UMETA(ToolTip = "Per-component minimum between current layer A and previous layer B."),
	Max UMETA(ToolTip = "Per-component maximum between current layer A and previous layer B."),
	In UMETA(ToolTip = "Current layer A masked by the previous layer B's alpha: A * b."),
	Mask UMETA(ToolTip = "Current layer A's alpha masking the previous layer B: B * a."),
	Screen UMETA(ToolTip = "Screen advanced blend mode."),
	Overlay UMETA(ToolTip = "Overlay advanced blend mode."),
	Darken UMETA(ToolTip = "Darken advanced blend mode."),
	Lighten UMETA(ToolTip = "Lighten advanced blend mode."),
	ColorDodge UMETA(ToolTip = "Color dodge advanced blend mode."),
	ColorBurn UMETA(ToolTip = "Color burn advanced blend mode."),
	HardLight UMETA(ToolTip = "Hard light advanced blend mode."),
	SoftLight UMETA(ToolTip = "Soft light advanced blend mode."),
	Difference UMETA(ToolTip = "Difference advanced blend mode."),
	Exclusion UMETA(ToolTip = "Exclusion advanced blend mode."),
};

namespace UE
{
	namespace CompositeCore
	{
		/** Describe whether lens distortion is handled in the merge shader pass. */
		enum class ELensDistortionHandling : uint8
		{
			Disabled = 0u,
			Enabled = 1u
		};

		class FMergePassProxy : public FCompositeCorePassProxy
		{
		public:
			IMPLEMENT_COMPOSITE_PASS(FMergePassProxy);

			using FCompositeCorePassProxy::FCompositeCorePassProxy;

			/** Constructor */
			UE_API FMergePassProxy(FPassInputDeclArray InPassDeclaredInputs, ECompositeCoreMergeOp InMergeOp, const TCHAR* InParentName, ELensDistortionHandling bInUseLensDistortion = ELensDistortionHandling::Enabled, bool bInUltimatteBlend = false);

			UE_API FPassTexture Add(FRDGBuilder& GraphBuilder, const FSceneView& InView, const FPassInputArray& Inputs, const FPassContext& PassContext) const override;

		public:
			/** Merge operation. */
			ECompositeCoreMergeOp MergeOp = ECompositeCoreMergeOp::Over;

			/** Parent pass debug name. */
			const TCHAR* ParentName = TEXT("MergePass");

			/** Flag to enable or disable application of engine-provided lens distortion LUTs, true by default. */
			ELensDistortionHandling bUseLensDistortion = ELensDistortionHandling::Enabled;

			/** When enabled, both inputs are encoded to sRGB before blending and decoded back to linear after. Matches Ultimatte Key & Fill workflows where content is in display space. */
			bool bUltimatteBlend = false;

			/**
			 * Divide-only: when true, the matte's RGB uses a single luminance ratio applied uniformly to all channels
			 * instead of independent per-channel ratios.
			 */
			bool bLuminanceOnly = false;
		};

		const FName& GetMergePassPassName();
	}
}

#undef UE_API
