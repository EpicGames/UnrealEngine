// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/Array.h"
#include "Containers/ContainersFwd.h"
#include "Misc/Optional.h"
#include "Misc/TVariant.h"
#include "SceneViewExtension.h"
#include "SceneTexturesConfig.h"
#include "ScreenPass.h"

#define UE_API COMPOSITECORE_API

class FRDGBuilder;
struct FRDGTextureDesc;
class FSceneView;
struct FScreenPassTexture;
struct FScreenPassRenderTarget;
struct FPostProcessMaterialInputs;
class FCompositeCorePassProxy;
class FRHISamplerState;
struct IPooledRenderTarget;
class UTexture;
class USceneCaptureComponent2D;
class AActor;
class FRenderTarget;

enum class EPostProcessMaterialInput : uint32;

namespace UE
{
	namespace CompositeCore
	{
		/** Texture encoding type, used for scene color. (HDR is not currently supported.) */
		enum class EEncoding : uint8
		{
			Linear = 0,
			Gamma = 1,
			sRGB = 2,
		};

		/**
		 * Strongly-typed identifier for passes, textures, or built-in renderer sources/targets.
		 *
		 * Values in [0, ExternalRangeStart) are reserved for built-in identifiers.
		 * Values >= ExternalRangeStart are allocated sequentially for user-declared external textures;
		 * the underlying index is offset so that ExternalInputs[0] corresponds to ExternalRangeStart.
		 *
		 * Underlying type is uint32 and values are stable; enumerators must not be renumbered since
		 * they may be referenced from serialized state or by external plugins.
		 */
		enum class ResourceId : uint32
		{
			/** Sentinel value; not a valid resource. */
			None = 0,
			/** Built-in custom render pass identifier. */
			BuiltInCRP = 1,
			/** Built-in empty/black identifier. */
			BuiltInEmpty = 2,
			/** First identifier of the external texture inputs, see ExternalInputs on FRenderWork. */
			ExternalRangeStart = 100,
		};

		/**
		 * Builds a ResourceId that identifies the external texture at the given array index in
		 * FRenderWork::ExternalInputs. This is the only supported way to construct an external
		 * ResourceId; callers should never compute one from raw integer arithmetic.
		 */
		constexpr ResourceId MakeExternalResourceId(int32 ExternalIndex)
		{
			return static_cast<ResourceId>(static_cast<uint32>(ResourceId::ExternalRangeStart) + static_cast<uint32>(ExternalIndex));
		}

		/**
		 * If Id refers to an external texture (i.e., Id >= ExternalRangeStart), returns the
		 * corresponding array index into FRenderWork::ExternalInputs. Returns an unset optional
		 * for built-in identifiers (None, BuiltInCRP, BuiltInEmpty).
		 */
		constexpr TOptional<int32> TryGetExternalIndex(ResourceId Id)
		{
			const uint32 Raw = static_cast<uint32>(Id);
			const uint32 Start = static_cast<uint32>(ResourceId::ExternalRangeStart);
			return (Raw >= Start) ? TOptional<int32>(static_cast<int32>(Raw - Start)) : TOptional<int32>();
		}

		/** Returns the underlying integer value for a ResourceId; use for logging only. */
		constexpr uint32 ToIndex(ResourceId Id) { return static_cast<uint32>(Id); }

		/** Texture resource metadata. */
		struct FResourceMetadata
		{
			/** Is the alpha inverted (like scene color)? */
			bool bInvertedAlpha = false;

			/** Is the texture content distorted? */
			bool bDistorted = false;

			/** Is the texture's exposure already adjusted? */
			bool bPreExposed = false;
			
			/** Source color encoding. */
			EEncoding Encoding = EEncoding::Linear;

			/** Sampler filter mode for the texture resource. */
			ESamplerFilter Filter = SF_Bilinear;

			/** Returns the RHI sampler state matching the Filter setting. */
			COMPOSITECORE_API FRHISamplerState* GetSamplerState() const;

			/** Debug name */
			const TCHAR* DebugName = TEXT("CompositeTexture");

			/** Equality operator */
			bool operator==(const FResourceMetadata& Other) const
			{
				return bInvertedAlpha == Other.bInvertedAlpha
					&& bDistorted == Other.bDistorted
					&& bPreExposed == Other.bPreExposed
					&& Encoding == Other.Encoding
					&& Filter == Other.Filter;

				// DebugName is intentionally ignored
			}
		};

		/** Pass texture description for internal resources (default scene textures). */
		struct FPassInternalResourceDesc
		{
			/** Index, which maps to the default 0-4 post-processing inputs or beyond. */
			int32 Index = 0;
			
			/** Flag to bypass the previous pass textures & access the original scene textures. */
			bool bOriginalCopyBeforePasses = false;
		};

		/** Pass texture description for external render targets. */
		struct FPassExternalResourceDesc
		{
			/** External resource identifier. */
			ResourceId Id = ResourceId::None;
		};

		/** Pass input declaration, referring to internal textures, external textures or the output of another pass proxy. */
		using FPassInputDecl = TVariant<FPassInternalResourceDesc, FPassExternalResourceDesc, const FCompositeCorePassProxy*>;

		/** Array of pass input declarations. */
		using FPassInputDeclArray = TArray<FPassInputDecl>;

		/** Returns a default input array declaring one internal texture resource. */
		UE_API FPassInputDeclArray GetDefaultInputDeclArray();

		/**
		 * Factory helpers for building FPassInputDecl values.
		 *
		 * These are preferred over direct variant construction (TInPlaceType<...> / .Set<T>(...))
		 * because they communicate intent and keep call sites terse.
		 */

		/** Build an input declaration referencing an internal post-processing texture by index. */
		UE_API FPassInputDecl MakeInternalInput(int32 Index = 0, bool bOriginalCopyBeforePasses = false);

		/** Build an input declaration referencing an external texture by ResourceId. */
		UE_API FPassInputDecl MakeExternalInput(ResourceId Id);

		/** Build an input declaration referencing the output of another pass proxy. */
		UE_API FPassInputDecl MakePassInput(const FCompositeCorePassProxy* Pass);

		/** External texture resource and its accompanying metadata. */
		struct FExternalTexture
		{
			/** Texture weak object pointer, for use on the game thread. */
			TWeakObjectPtr<UTexture> Texture = {};
			
			/** Texture metadata. */
			FResourceMetadata Metadata = {};
		};

		/** Resolved texture resource with an active (screen) texture and its accompanying metadata. */
		struct FPassTexture
		{
			/** Pass screen texture. */
			FScreenPassTexture Texture = {};
			
			/** Texture metadata. */
			FResourceMetadata Metadata = {};
		};

		/** Pass texture input definition. */
		using FPassInput = FPassTexture;

		/**
		 * Resolved pass inputs handed to FCompositeCorePassProxy::Add at render time.
		 *
		 * Render-thread, frame-scoped. Inputs are resolved from declared inputs (internal textures,
		 * external textures, or previous pass outputs) by the scene view extension before dispatch.
		 */
		struct FPassInputArray
		{
			/** Default constructor */
			FPassInputArray() = default;
			FPassInputArray(FPassInputArray&&) = default;
			FPassInputArray(const FPassInputArray&) = default;
			FPassInputArray& operator=(FPassInputArray&&) = default;
			FPassInputArray& operator=(const FPassInputArray&) = default;

			/** Constructor */
			UE_API FPassInputArray(
				FRDGBuilder& GraphBuilder,
				const FSceneView& InView,
				const FPostProcessMaterialInputs& InPostInputs,
				const ISceneViewExtension::EPostProcessingPass& InLocation
			);

			/** Array access operator overload */
			const FPassInput& operator[] (int32 Index) const
			{
				return Inputs[Index];
			}

			/** Array access operator overload */
			FPassInput& operator[] (int32 Index)
			{
				return Inputs[Index];
			}

			/** Tests if index is valid, i.e. greater than or equal to zero, and less than the number of elements in the array. */
			bool IsValidIndex(int32 Index) const
			{
				return Inputs.IsValidIndex(Index);
			}

			/** Number of pass inputs */
			int32 Num() const { return Inputs.Num(); }

			/** Mutable access to the underlying input array. Prefer the subscript operator for read-only access. */
			TArray<UE::CompositeCore::FPassInput>& GetInputs() { return Inputs; }

			/** Read-only view of the underlying input array. */
			TConstArrayView<UE::CompositeCore::FPassInput> GetInputs() const { return Inputs; }

			/** Conversion function to engine post-process (material) inputs. */
			UE_API FPostProcessMaterialInputs ToPostProcessInputs(FRDGBuilder& GraphBuilder, FSceneTextureShaderParameters SceneTextures) const;

			/** Pass override output. */
			FScreenPassRenderTarget OverrideOutput;

		private:
			/** Pass texture input array. */
			TArray<UE::CompositeCore::FPassInput> Inputs;
		};

		/**
		 * Per-dispatch pass parameter information.
		 *
		 * Render-thread, frame-scoped; passed by value/const-ref into FCompositeCorePassProxy::Add.
		 */
		struct FPassContext
		{
			/** The uniform buffer containing all scene textures. */
			FSceneTextureShaderParameters SceneTextures = {};

			/** Active post-processing output view rectangle. */
			FIntRect OutputViewRect = {};

			/** Post-processing location in the scene view extension pipeline. */
			ISceneViewExtension::EPostProcessingPass Location = ISceneViewExtension::EPostProcessingPass::BeforeDOF;

			/** Is the current pass expected to output scene color? */
			bool bOutputSceneColor = false;
		};

		/** Options to control the built-in custom render pass. */
		struct FBuiltInRenderPassOptions final
		{
			/** Constructor. DilationSize is initialized from the CompositeCore.Debug.DilationSize CVar. */
			UE_API FBuiltInRenderPassOptions();

			/** Custom user flags value used to alter materials in the composite render pass. */
			TOptional<int32> ViewUserFlagsOverride;

			/** Enables the development shader debug feature that routes the Base Color output to Emissive for the separate render. Non-shipping PC build only. */
			bool bEnableUnlitViewmode = true;

			/** Dilation kernel radius (0 = no-op, 1 = 3×3 neighborhood, 2 = 5×5 neighborhood). Clamped to [0, 2]. Default comes from CompositeCore.Debug.DilationSize. */
			int32 DilationSize = 1;

			/** Opacify to extract the solid colors behind translucent alpha holdout masks. */
			bool bOpacifyOutput = true;
		};

		/**
		 * Per-frame filter restricting compositing to a single viewport. Populated by the host
		 * (see ACompositeActor) when ACompositeActor::bRestrictToActiveViewport is enabled, and
		 * stored as a TOptional so the unset state represents "no restriction".
		 * TargetRenderTarget is used for pointer comparison only and never dereferenced.
		 *
		 * Read only on the game thread today (see SVE::SetupView). Any future render-thread
		 * consumer should reconsider TWeakObjectPtr / raw-pointer thread-safety before reading.
		 */
		struct FActiveViewportFilter
		{
			/** Target view actor, matched against FSceneView::ViewActor. */
			TWeakObjectPtr<const AActor> TargetViewActor;

			/** Target viewport render target (editor only). Pointer comparison only. */
			const FRenderTarget* TargetRenderTarget = nullptr;
		};

		/**
		 * Render-thread struct for scene view extension render work per frame.
		 *
		 * Populated on the game thread (typically via UCompositeCoreSubsystem::SetRenderWork)
		 * and handed off via move. After handoff the struct is treated as immutable on the
		 * render thread for the duration of the frame. All FCompositeCorePassProxy pointers
		 * referenced by PreprocessingPasses and FramePasses must be allocated from FrameAllocator
		 * and must not outlive it.
		 */
		struct FRenderWork final
		{
			/** Default frame work. */
			static UE_API const FRenderWork& GetDefault();

			/** Constructor. */
			UE_API FRenderWork();

			/** Returns true if there are no active work passes. */
			bool IsEmpty() const;

			/** Array of user-defined external input texture overrides, where array index + ResourceId::ExternalRangeStart maps to the ResourceId. */
			TArray<FExternalTexture> ExternalInputs;

			/** Pre-processing passes applied on specific texture resources at the start of rendering. Passes within a value array execute strictly in array order. */
			TSortedMap<ResourceId, TArray<const FCompositeCorePassProxy*>> PreprocessingPasses;

			/** Post-processing passes at the specified pipeline locations. Passes within a value array execute strictly in array order; producers are responsible for emitting a valid order. */
			TSortedMap<ISceneViewExtension::EPostProcessingPass, TArray<const FCompositeCorePassProxy*>> FramePasses;

			/** Proxy allocator. Owns the lifetime of all FCompositeCorePassProxy pointers in this struct. */
			TUniquePtr<FSceneRenderingBulkObjectAllocator> FrameAllocator;

			/** Optional main render mode override */
			TOptional<ESceneCaptureSource> MainRenderMode;

			/** View modes for which the compositing is allowed. */
			TSet<EViewModeIndex> AllowedViewModes;

			/** Optional per-frame filter restricting compositing to a single viewport (see FActiveViewportFilter). */
			TOptional<FActiveViewportFilter> ActiveViewportFilter;

			/** List of scene captures to render-update this frame. */
			TArray<TWeakObjectPtr<USceneCaptureComponent2D>> SceneCapturesUpdateQueue;
		};

		/**
		 * Static metadata describing the shape of a pass type.
		 *
		 * Surfaced by FCompositeCorePassProxy::GetTypeDescriptor. Intended to let tooling
		 * (future graph UI, validation, serialization) reason about a pass without dispatching it.
		 *
		 * This struct is intentionally not final: new descriptor fields may be added in the future
		 * (e.g., acceptable input variant kinds per slot, output slot metadata) without breaking
		 * existing consumers.
		 */
		struct FPassTypeDescriptor
		{
			/** Unique type name; matches FCompositeCorePassProxy::GetTypeName. */
			FName TypeName;

			/**
			 * Stable names for declared input slots, in declaration order.
			 *
			 * These are the identity used when a graph UI, inspector, or serialized asset needs
			 * to reference a specific input independent of its array index. If left empty, tooling
			 * should synthesize fallback names ("Input0", "Input1", ...) from GetNumDeclaredInputs.
			 */
			TArray<FName> InputSlotNames;

			/**
			 * Fixed input count, if the pass accepts exactly this many inputs (e.g., two-input merge).
			 * Unset means the pass accepts a variable number of inputs.
			 */
			TOptional<int32> FixedInputCount;

			/** True if the pass can accept more than FixedInputCount inputs (variadic). */
			bool bSupportsVariadicInputs = false;
		};
	}
}

/** Render-thread pass proxy. */
class FCompositeCorePassProxy
{
public:
	/** Default constructor. Intended for use by subclasses that populate declared inputs later. */
	FCompositeCorePassProxy() = default;

	/** Constructor taking the pass's declared inputs. */
	UE_API FCompositeCorePassProxy(UE::CompositeCore::FPassInputDeclArray InPassDeclaredInputs);

	/** Virtual destructor */
	virtual ~FCompositeCorePassProxy() = default;

	/** Bare-bones RTTI method to allow users to differentiate / downcast the composite core pass they're interested in. */
	virtual const FName& GetTypeName() const = 0;

	/**
	 * Returns static metadata about this pass type.
	 *
	 * The default implementation synthesizes a descriptor from GetTypeName() and the currently
	 * declared inputs (with fallback slot names "Input0", "Input1", ...). Subclasses should
	 * override this to provide meaningful slot names and arity hints that tooling can rely on.
	 */
	UE_API virtual UE::CompositeCore::FPassTypeDescriptor GetTypeDescriptor() const;

	/** Render-thread add pass method to override. */
	virtual UE::CompositeCore::FPassTexture Add(
		FRDGBuilder& GraphBuilder,
		const FSceneView& InView,
		const UE::CompositeCore::FPassInputArray& Inputs,
		const UE::CompositeCore::FPassContext& PassContext
	) const = 0;

	/** Number of inputs used by the pass. */
	int32 GetNumDeclaredInputs() const
	{
		return PassDeclaredInputs.Num();
	}

	/** Get pass input at specified index. */
	const UE::CompositeCore::FPassInputDecl& GetDeclaredInput(int32 InputIndex) const
	{
		return PassDeclaredInputs[InputIndex];
	}

	/**
	 * Replaces the declared input at the specified index.
	 *
	 * The index must be in range [0, GetNumDeclaredInputs()). This intentionally does not resize
	 * the declared-input array — arity is fixed at construction time. Used by layer-traversal
	 * code to back-patch a declared input after its source pass has been constructed.
	 */
	UE_API void SetDeclaredInput(int32 InputIndex, UE::CompositeCore::FPassInputDecl InInput);

	/** Get declared primary output pass override. */
	const TOptional<UE::CompositeCore::ResourceId>& GetDeclaredPrimaryOutputOverride() const
	{
		return PassDeclaredPrimaryOutputOverride;
	}

	/**
	 * Declares that the primary output of this pass should be written to the given external resource.
	 *
	 * "Primary" is used intentionally to leave semantic room for secondary/additional outputs
	 * without a breaking rename in a future revision.
	 */
	void DeclarePrimaryOutputOverride(UE::CompositeCore::ResourceId InExternalId)
	{
		PassDeclaredPrimaryOutputOverride = InExternalId;
	}

	/** Resets the declared primary output override. */
	void ResetPrimaryOutputOverride()
	{
		PassDeclaredPrimaryOutputOverride.Reset();
	}

	/** Convenience function to create an output render target with the specified resolution. */
	static UE_API FScreenPassRenderTarget CreateOutputRenderTarget(FRDGBuilder& GraphBuilder,
		const FSceneView& InView, const FIntRect& OutputViewRect, FRDGTextureDesc OutputDesc, const TCHAR* InName);

	/**
	 * Returns FSceneTexturesConfig::Get().ColorFormat, or Fallback if the config has not been
	 * initialized for the current view.
	 */
	static UE_API EPixelFormat GetSceneColorFormatChecked(EPixelFormat Fallback = PF_FloatRGBA);

protected:

	/**
	 * Ensures that the received inputs are compatible with the declared inputs and the type descriptor.
	 *
	 * Returns false if the input count violates the descriptor's FixedInputCount (when set and
	 * bSupportsVariadicInputs is false). Otherwise requires Inputs.Num() >= max(1, declared count).
	 */
	UE_API bool ValidateInputs(const UE::CompositeCore::FPassInputArray& Inputs) const;

	/** List of pass input types. */
	UE::CompositeCore::FPassInputDeclArray PassDeclaredInputs;

	/** Optional declared primary output pass override. */
	TOptional<UE::CompositeCore::ResourceId> PassDeclaredPrimaryOutputOverride;
};

/**
 * Use this macro to implement the RTTI method for composite render passes.
 *
 * Should only be used on final classes (e.g. if C derives from B which derives from A, we won't
 * be able to query a C* for whether it's a "B"; in other words, A and B should be abstract and C,
 * final).
 *
 * Expands to an override of GetTypeName() plus a static GetTypeNameStatic() accessor usable for
 * type comparison without an instance.
 */
#define IMPLEMENT_COMPOSITE_PASS(TypeName) const FName& GetTypeName() const override { return GetTypeNameStatic(); } \
	static const FName& GetTypeNameStatic() { static FName Name(TEXT(#TypeName)); return Name; }

#undef UE_API
